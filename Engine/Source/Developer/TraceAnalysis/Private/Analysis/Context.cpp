// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processor.h"
#include "Trace/Analysis.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
void FAnalysisContext::AddAnalyzer(IAnalyzer& Analyzer)
{
	Analyzers.Add(&Analyzer);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisContext::SetMessageDelegate(FMessageDelegate Delegate)
{
	OnMessage = Delegate;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisContext::SetOnCompletedDelegate(FOnCompletedDelegate Delegate)
{
	OnCompleted = Delegate;
}

////////////////////////////////////////////////////////////////////////////////
FAnalysisProcessor FAnalysisContext::Process(IInDataStream& DataStream)
{
	FAnalysisProcessor Processor;
	if (Analyzers.Num() > 0)
	{
		Processor.Impl = new FAnalysisProcessor::FImpl(DataStream, MoveTemp(Analyzers), MoveTemp(OnMessage), MoveTemp(OnCompleted));
	}
	return MoveTemp(Processor);
}

} // namespace Trace
} // namespace UE
