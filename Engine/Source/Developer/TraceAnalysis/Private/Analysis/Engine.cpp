// Copyright Epic Games, Inc. All Rights Reserved.

#include "Analysis/Engine.h"

#include "Algo/BinarySearch.h"
#include "Algo/Sort.h"
#include "Algo/StableSort.h"
#include "Containers/ArrayView.h"
#include "CoreGlobals.h"
#include "HAL/UnrealMemory.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogMacros.h"
#include "Misc/StringBuilder.h"
#include "StreamReader.h"
#include "Templates/UnrealTemplate.h"
#include "TraceAnalysisDebug.h"
#include "Trace/Analysis.h"
#include "Trace/Analyzer.h"
#include "Trace/Detail/Protocol.h"
#include "Trace/Detail/Transport.h"
#include "Transport/PacketTransport.h"
#include "Transport/TidPacketTransport.h"
#include "Transport/Transport.h"

DEFINE_LOG_CATEGORY_STATIC(LogTraceAnalysis, Log, All)

#define LOCTEXT_NAMESPACE "TraceAnalysis"

namespace UE {
namespace Trace {

// {{{1 misc -------------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
void SerializeToCborImpl(TArray<uint8>&, const IAnalyzer::FEventData&, uint32);

////////////////////////////////////////////////////////////////////////////////
class FFnv1aHash
{
public:
			FFnv1aHash() = default;
			FFnv1aHash(uint32 PrevResult)		{ Result = PrevResult; }
	void	Add(const ANSICHAR* String)			{ for (; *String; ++String) { Add(*String); } }
	void	Add(const uint8* Data, uint32 Size)	{ for (uint32 i = 0; i < Size; ++Data, ++i) { Add(*Data); } }
	void	Add(uint8 Value)					{ Result ^= Value; Result *= 0x01000193; }
	uint32	Get() const							{ return Result; }

private:
	uint32	Result = 0x811c9dc5;
	// uint32: bias=0x811c9dc5			prime=0x01000193
	// uint64: bias=0xcbf29ce484222325	prime=0x00000100000001b3;
};



// {{{1 aux-data ---------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
struct FAuxData
{
	const uint8*	Data;
	uint32			DataSize;
	uint16			FieldIndex;
	int8			FieldSizeAndType;
	uint8			bSigned		: 1;
	uint8			bFragmented	: 1;
	uint8			bOwnsData	: 1;
	uint8			_Unused		: 5;
};

////////////////////////////////////////////////////////////////////////////////
struct FAuxDataCollector
	: public TArray<FAuxData, TInlineAllocator<8>>
{
	using	Super = TArray<FAuxData, TInlineAllocator<8>>;

			~FAuxDataCollector() { Reset(); }
	void	Add(const FAuxData& Data);
	void	Defragment(FAuxData& Head);
	void	Reset();
};

////////////////////////////////////////////////////////////////////////////////
void FAuxDataCollector::Add(const FAuxData& Data)
{
	if (Num() == 0)
	{
		Super::Add(Data);
		return;
	}

	FAuxData* Prev = &Last();
	if (Prev->FieldIndex != Data.FieldIndex)
	{
		Super::Add(Data);
		return;
	}

	while (Prev > GetData())
	{
		if (Prev[-1].FieldIndex != Data.FieldIndex)
		{
			break;
		}

		--Prev;
	}

	Prev->bFragmented = 1;
	Super::Add(Data);
}

////////////////////////////////////////////////////////////////////////////////
void FAuxDataCollector::Defragment(FAuxData& Data)
{
	check(Data.bFragmented);

	uint32 Size = Data.DataSize;
	for (FAuxData* Read = &Data + 1;;)
	{
		Size += Read->DataSize;

		++Read;
		if (Read > &Last() || Read->FieldIndex != Data.FieldIndex)
		{
			break;
		}
	}

	uint8* Buffer = (uint8*)(FMemory::Malloc(Size));

	uint8* Write = Buffer;
	uint8* End = Buffer + Size;
	for (FAuxData* Read = &Data;; ++Read)
	{
		FMemory::Memcpy(Write, Read->Data, Read->DataSize);
		Write += Read->DataSize;
		if (Write >= End)
		{
			break;
		}
	}

	Data.Data = Buffer;
	Data.DataSize = Size;
	Data.bOwnsData = 1;
	Data.bFragmented = 0;
}

////////////////////////////////////////////////////////////////////////////////
void FAuxDataCollector::Reset()
{
	for (const FAuxData& AuxData : *this)
	{
		if (!AuxData.bOwnsData)
		{
			continue;
		}

		FMemory::Free((void*)(AuxData.Data));
	}

	Super::Reset();
}



// {{{1 threads ----------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FThreads
{
public:
	struct FInfo
	{
		TArray<uint64>	ScopeRoutes;
		TArray<uint8>	Name;
		TArray<uint8>	GroupName;
		int64			PrevTimestamp = 0;
		uint32			ThreadId = ~0u;
		uint32			SystemId = 0;
		int32			SortHint = 0xff;
	};

						FThreads();
						~FThreads();
	FInfo*				GetInfo();
	FInfo*				GetInfo(uint32 ThreadId);
	void				SetGroupName(const ANSICHAR* InGroupName, uint32 Length);
	const TArray<uint8>*GetGroupName() const;

private:
	uint32				LastGetInfoId = ~0u;
	TArray<FInfo*>		Infos;
	TArray<uint8>		GroupName;
};

////////////////////////////////////////////////////////////////////////////////
FThreads::FThreads()
{
	Infos.Reserve(64);
}

////////////////////////////////////////////////////////////////////////////////
FThreads::~FThreads()
{
	for (FInfo* Info : Infos)
	{
		delete Info;
	}
}

////////////////////////////////////////////////////////////////////////////////
FThreads::FInfo* FThreads::GetInfo()
{
	return GetInfo(LastGetInfoId);
}

////////////////////////////////////////////////////////////////////////////////
FThreads::FInfo* FThreads::GetInfo(uint32 ThreadId)
{
	LastGetInfoId = ThreadId;

	if (ThreadId >= uint32(Infos.Num()))
	{
		Infos.SetNumZeroed(ThreadId + 1);
	}

	FInfo* Info = Infos[ThreadId];
	if (Info == nullptr)
	{
		Info = new FInfo();
		Info->ThreadId = ThreadId;
		Infos[ThreadId] = Info;
	}
	return Info;
}

////////////////////////////////////////////////////////////////////////////////
void FThreads::SetGroupName(const ANSICHAR* InGroupName, uint32 Length)
{
	if (InGroupName == nullptr || *InGroupName == '\0')
	{
		GroupName.SetNum(0);
		return;
	}

	GroupName.SetNumUninitialized(Length + 1);
	GroupName[Length] = '\0';
	FMemory::Memcpy(GroupName.GetData(), InGroupName, Length);
}

////////////////////////////////////////////////////////////////////////////////
const TArray<uint8>* FThreads::GetGroupName() const
{
	return (GroupName.Num() > 0) ? &GroupName : nullptr;
}



// {{{1 thread-info ------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
uint32 IAnalyzer::FThreadInfo::GetId() const
{
	const auto* Info = (const FThreads::FInfo*)this;
	return Info->ThreadId;
}

////////////////////////////////////////////////////////////////////////////////
uint32 IAnalyzer::FThreadInfo::GetSystemId() const
{
	const auto* Info = (const FThreads::FInfo*)this;
	return Info->SystemId;
}

////////////////////////////////////////////////////////////////////////////////
int32 IAnalyzer::FThreadInfo::GetSortHint() const
{
	const auto* Info = (const FThreads::FInfo*)this;
	return Info->SortHint;
}

////////////////////////////////////////////////////////////////////////////////
const ANSICHAR* IAnalyzer::FThreadInfo::GetName() const
{
	const auto* Info = (const FThreads::FInfo*)this;
	if (Info->Name.Num() <= 0)
	{
		return nullptr;
	}

	return (const ANSICHAR*)(Info->Name.GetData());
}

////////////////////////////////////////////////////////////////////////////////
const ANSICHAR* IAnalyzer::FThreadInfo::GetGroupName() const
{
	const auto* Info = (const FThreads::FInfo*)this;
	if (Info->GroupName.Num() <= 0)
	{
		return "";
	}

	return (const ANSICHAR*)(Info->GroupName.GetData());
}



// {{{1 timing -----------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
struct FTiming
{
	uint64	BaseTimestamp = 0;
	uint64	TimestampHz = 0;
	double	InvTimestampHz = 0.0;
	uint64	EventTimestamp = 0;
	double  BaseDateTimestamp = 0.0;
};



////////////////////////////////////////////////////////////////////////////////
uint64 IAnalyzer::FEventTime::GetTimestamp() const
{
	const auto* Timing = (const FTiming*)this;
	return Timing->EventTimestamp;
}

////////////////////////////////////////////////////////////////////////////////
double IAnalyzer::FEventTime::AsSeconds() const
{
	const auto* Timing = (const FTiming*)this;
	return double(Timing->EventTimestamp) * Timing->InvTimestampHz;
}

////////////////////////////////////////////////////////////////////////////////
uint64 IAnalyzer::FEventTime::AsCycle64() const
{
	const auto* Timing = (const FTiming*)this;
	return Timing->BaseTimestamp + Timing->EventTimestamp;
}

////////////////////////////////////////////////////////////////////////////////
double IAnalyzer::FEventTime::AsSeconds(uint64 Cycles64) const
{
	const auto* Timing = (const FTiming*)this;
	return double(int64(Cycles64) - int64(Timing->BaseTimestamp)) * Timing->InvTimestampHz;
}

////////////////////////////////////////////////////////////////////////////////
double IAnalyzer::FEventTime::AsSecondsAbsolute(int64 DurationCycles64) const
{
	const auto* Timing = (const FTiming*)this;
	return double(DurationCycles64) * Timing->InvTimestampHz;
}

////////////////////////////////////////////////////////////////////////////////
double IAnalyzer::FEventTime::GetBaseDateTimestamp() const
{
	const auto* Timing = (const FTiming*)this;
	return Timing->BaseDateTimestamp;
}


// {{{1 dispatch ---------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
struct FDispatch
{
	struct FField
	{
		uint32		Hash;
		uint16		Offset;
		uint16		Size;
		uint16		NameOffset;			// From FField ptr
		int8		SizeAndType;		// value == byte_size, sign == float < 0 < int
		uint8		Class : 7;
		uint8		bArray : 1;
		uint32		RefUid;				// If reference field, uid of ref type
	};

	int32			GetFieldIndex(const ANSICHAR* Name) const;
	uint32			Hash				= 0;
	uint16			Uid					= 0;
	uint8			FieldCount			= 0;
	uint8			Flags				= 0;	// IAnalyzer::EEventFlags
	uint16			EventSize			= 0;
	uint16			LoggerNameOffset	= 0;	// From FDispatch ptr
	uint16			EventNameOffset		= 0;	// From FDispatch ptr
	FField			Fields[];

	inline bool IsImportant() const  { return (Flags & (uint8)IAnalyzer::EEventFlags::Important) != 0; }
	inline bool IsNoSync() const     { return (Flags & (uint8)IAnalyzer::EEventFlags::NoSync) != 0; }
	inline bool IsSync() const       { return (Flags & (uint8)IAnalyzer::EEventFlags::NoSync) == 0; }
	inline bool IsDefinition() const { return (Flags & (uint8)IAnalyzer::EEventFlags::Definition) != 0; }
	inline bool MaybeHasAux() const  { return (Flags & (uint8)IAnalyzer::EEventFlags::MaybeHasAux) != 0; }
};

////////////////////////////////////////////////////////////////////////////////
int32 FDispatch::GetFieldIndex(const ANSICHAR* Name) const
{
	FFnv1aHash NameHash;
	NameHash.Add(Name);

	for (int32 i = 0, n = FieldCount; i < n; ++i)
	{
		if (Fields[i].Hash == NameHash.Get())
		{
			return i;
		}
	}

	return -1;
}



// {{{1 dispatch-builder -------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FDispatchBuilder
{
public:
						FDispatchBuilder();
	void				SetUid(uint16 Uid);
	void				SetLoggerName(const ANSICHAR* Name, int32 NameSize=-1);
	void				SetEventName(const ANSICHAR* Name, int32 NameSize=-1);
	void				SetFlags(IAnalyzer::EEventFlags Flags);
	FDispatch::FField&	AddField(const ANSICHAR* Name, int32 NameSize, uint16 Size);
	FDispatch*			Finalize();

private:
	uint32				AppendName(const ANSICHAR* Name, int32 NameSize);
	TArray<uint8>		Buffer;
	TArray<uint8>		NameBuffer;
};

////////////////////////////////////////////////////////////////////////////////
FDispatchBuilder::FDispatchBuilder()
{
	Buffer.SetNum(sizeof(FDispatch));

	auto* Dispatch = (FDispatch*)(Buffer.GetData());
	new (Dispatch) FDispatch();
}

////////////////////////////////////////////////////////////////////////////////
FDispatch* FDispatchBuilder::Finalize()
{
	int32 Size = Buffer.Num() + NameBuffer.Num();
	auto* Dispatch = (FDispatch*)FMemory::Malloc(Size);
	memcpy(Dispatch, Buffer.GetData(), Buffer.Num());
	memcpy(Dispatch->Fields + Dispatch->FieldCount, NameBuffer.GetData(), NameBuffer.Num());

	// Fix up name offsets
	for (int i = 0, n = Dispatch->FieldCount; i < n; ++i)
	{
		auto* Field = Dispatch->Fields + i;
		Field->NameOffset += (uint16)(Buffer.Num() - uint32(UPTRINT(Field) - UPTRINT(Dispatch)));
	}

	// Calculate this dispatch's hash.
	if (Dispatch->LoggerNameOffset || Dispatch->EventNameOffset)
	{
		Dispatch->LoggerNameOffset += (uint16)Buffer.Num();
		Dispatch->EventNameOffset += (uint16)Buffer.Num();

		FFnv1aHash Hash;
		Hash.Add((const ANSICHAR*)Dispatch + Dispatch->LoggerNameOffset);
		Hash.Add((const ANSICHAR*)Dispatch + Dispatch->EventNameOffset);
		Dispatch->Hash = Hash.Get();
	}

	return Dispatch;
}

////////////////////////////////////////////////////////////////////////////////
void FDispatchBuilder::SetUid(uint16 Uid)
{
	auto* Dispatch = (FDispatch*)(Buffer.GetData());
	Dispatch->Uid = Uid;
}

////////////////////////////////////////////////////////////////////////////////
void FDispatchBuilder::SetLoggerName(const ANSICHAR* Name, int32 NameSize)
{
	auto* Dispatch = (FDispatch*)(Buffer.GetData());
	Dispatch->LoggerNameOffset += (uint16)AppendName(Name, NameSize);
}

////////////////////////////////////////////////////////////////////////////////
void FDispatchBuilder::SetEventName(const ANSICHAR* Name, int32 NameSize)
{
	auto* Dispatch = (FDispatch*)(Buffer.GetData());
	Dispatch->EventNameOffset = (uint16)AppendName(Name, NameSize);
}

////////////////////////////////////////////////////////////////////////////////
void FDispatchBuilder::SetFlags(IAnalyzer::EEventFlags Flags)
{
	auto* Dispatch = (FDispatch*)(Buffer.GetData());
	Dispatch->Flags = (uint8)Flags;
}

////////////////////////////////////////////////////////////////////////////////
FDispatch::FField& FDispatchBuilder::AddField(const ANSICHAR* Name, int32 NameSize, uint16 Size)
{
	int32 Bufoff = Buffer.AddUninitialized(sizeof(FDispatch::FField));
	auto* Field = (FDispatch::FField*)(Buffer.GetData() + Bufoff);
	Field->NameOffset = (uint16)AppendName(Name, NameSize);
	Field->Size = Size;
	Field->RefUid = 0;

	FFnv1aHash Hash;
	Hash.Add((const uint8*)Name, NameSize);
	Field->Hash = Hash.Get();

	auto* Dispatch = (FDispatch*)(Buffer.GetData());
	Dispatch->FieldCount++;
	Dispatch->EventSize += Field->Size;

	return *Field;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FDispatchBuilder::AppendName(const ANSICHAR* Name, int32 NameSize)
{
	if (NameSize < 0)
	{
		NameSize = int32(FCStringAnsi::Strlen(Name));
	}

	int32 Ret = NameBuffer.AddUninitialized(NameSize + 1);
	uint8* Out = NameBuffer.GetData() + Ret;
	memcpy(Out, Name, NameSize);
	Out[NameSize] = '\0';
	return Ret;
}



// {{{1 event-type-info --------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
uint32 IAnalyzer::FEventTypeInfo::GetId() const
{
	const auto* Inner = (const FDispatch*)this;
	return Inner->Uid;
}

////////////////////////////////////////////////////////////////////////////////
IAnalyzer::EEventFlags IAnalyzer::FEventTypeInfo::GetFlags() const
{
	const auto* Inner = (const FDispatch*)this;
	return static_cast<IAnalyzer::EEventFlags>(Inner->Flags);
}

////////////////////////////////////////////////////////////////////////////////
const ANSICHAR* IAnalyzer::FEventTypeInfo::GetName() const
{
	const auto* Inner = (const FDispatch*)this;
	return (const ANSICHAR*)Inner + Inner->EventNameOffset;
}

////////////////////////////////////////////////////////////////////////////////
const ANSICHAR* IAnalyzer::FEventTypeInfo::GetLoggerName() const
{
	const auto* Inner = (const FDispatch*)this;
	return (const ANSICHAR*)Inner + Inner->LoggerNameOffset;
}

////////////////////////////////////////////////////////////////////////////////
uint32 IAnalyzer::FEventTypeInfo::GetSize() const
{
	const auto* Inner = (const FDispatch*)this;
	return Inner->EventSize;
}

////////////////////////////////////////////////////////////////////////////////
uint32 IAnalyzer::FEventTypeInfo::GetFieldCount() const
{
	const auto* Inner = (const FDispatch*)this;
	return Inner->FieldCount;
}

////////////////////////////////////////////////////////////////////////////////
const IAnalyzer::FEventFieldInfo* IAnalyzer::FEventTypeInfo::GetFieldInfo(uint32 Index) const
{
	if (Index >= GetFieldCount())
	{
		return nullptr;
	}

	const auto* Inner = (const FDispatch*)this;
	return (const IAnalyzer::FEventFieldInfo*)(Inner->Fields + Index);
}

////////////////////////////////////////////////////////////////////////////////
int32 IAnalyzer::FEventTypeInfo::GetFieldIndex(const ANSICHAR* FieldName) const
{
	const auto* Inner = (const FDispatch*)this;
	return Inner->GetFieldIndex(FieldName);
}

////////////////////////////////////////////////////////////////////////////////
IAnalyzer::FEventFieldHandle IAnalyzer::FEventTypeInfo::GetFieldHandleUnchecked(uint32 Index) const
{
	const auto* Inner = (const FDispatch*)this;
	if (Index >= Inner->FieldCount)
	{
		return FEventFieldHandle { -1 };
	}
	return FEventFieldHandle { Inner->Fields[Index].Offset };
}

////////////////////////////////////////////////////////////////////////////////
IAnalyzer::FEventFieldHandle IAnalyzer::FEventTypeInfo::GetFieldHandleImpl(
	const ANSICHAR* FieldName,
	int16& SizeAndType) const
{
	const auto* Inner = (const FDispatch*)this;
	int32 Index = Inner->GetFieldIndex(FieldName);
	if (Index < 0)
	{
		return { -1 };
	}

	const FDispatch::FField& Field = Inner->Fields[Index];
	SizeAndType = Field.SizeAndType;
	return { Field.Offset };
}



// {{{1 field-info -------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
const ANSICHAR* IAnalyzer::FEventFieldInfo::GetName() const
{
	const auto* Inner = (const FDispatch::FField*)this;
	return (const ANSICHAR*)(UPTRINT(Inner) + Inner->NameOffset);
}

////////////////////////////////////////////////////////////////////////////////
IAnalyzer::FEventFieldInfo::EType IAnalyzer::FEventFieldInfo::GetType() const
{
	const auto* Inner = (const FDispatch::FField*)this;

	if (Inner->RefUid != 0)
	{
		switch (Inner->Size)
		{
			case sizeof(uint8): return EType::Reference8;
			case sizeof(uint16): return EType::Reference16;
			case sizeof(uint32): return EType::Reference32;
			case sizeof(uint64): return EType::Reference64;
			default: check(false); // Unsupported width
		}
	}

	if (Inner->Class == UE::Trace::Protocol0::Field_String)
	{
		return (Inner->SizeAndType == 1) ? EType::AnsiString : EType::WideString;
	}

	if (Inner->SizeAndType > 0)
	{
		return EType::Integer;
	}

	if (Inner->SizeAndType < 0)
	{
		return EType::Float;
	}

	return EType::None;
}

////////////////////////////////////////////////////////////////////////////////
uint8 IAnalyzer::FEventFieldInfo::GetTypeSize() const
{
	const auto* Inner = (const FDispatch::FField*)this;

	if (Inner->RefUid != 0)
	{
		return uint8(Inner->Size);
	}

	if (Inner->Class == UE::Trace::Protocol0::Field_String)
	{
		return 0;
	}

	if (Inner->SizeAndType > 0)
	{
		return uint8(Inner->SizeAndType);
	}

	if (Inner->SizeAndType < 0)
	{
		return uint8(-Inner->SizeAndType);
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
bool IAnalyzer::FEventFieldInfo::IsArray() const
{
	const auto* Inner = (const FDispatch::FField*)this;
	return Inner->bArray;
}

////////////////////////////////////////////////////////////////////////////////
bool IAnalyzer::FEventFieldInfo::IsSigned() const
{
	const auto* Inner = (const FDispatch::FField*)this;
	return (Inner->Class & UE::Trace::Protocol0::Field_Signed) != 0;
}

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ANALYSIS_DEBUG_API
uint32 IAnalyzer::FEventFieldInfo::GetOffset() const
{
	const auto* Inner = (const FDispatch::FField*)this;
	return Inner->Offset;
}
#endif // UE_TRACE_ANALYSIS_DEBUG_API

////////////////////////////////////////////////////////////////////////////////
uint8 IAnalyzer::FEventFieldInfo::GetSize() const
{
	const auto* Inner = (const FDispatch::FField*)this;
	return uint8(Inner->Size);
}

////////////////////////////////////////////////////////////////////////////////
uint32 IAnalyzer::FEventFieldInfo::GetRefUid() const
{
	const auto* Inner = (const FDispatch::FField*)this;
	return Inner->RefUid;
}



// {{{1 array-reader -----------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
uint32 IAnalyzer::FArrayReader::Num() const
{
	const auto* Inner = (const FAuxData*)this;
	int32 SizeAndType = Inner->FieldSizeAndType;
	SizeAndType = (SizeAndType < 0) ? -SizeAndType : SizeAndType;
	return (SizeAndType == 0) ? SizeAndType : (Inner->DataSize / SizeAndType);
}

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ANALYSIS_DEBUG_API
const uint8* IAnalyzer::FArrayReader::GetRawData() const
{
	const auto* Inner = (const FAuxData*)this;
	return Inner->Data;
}
#endif // UE_TRACE_ANALYSIS_DEBUG_API

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ANALYSIS_DEBUG_API
uint32 IAnalyzer::FArrayReader::GetRawDataSize() const
{
	const auto* Inner = (const FAuxData*)this;
	return Inner->DataSize;
}
#endif // UE_TRACE_ANALYSIS_DEBUG_API

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ANALYSIS_DEBUG_API
int8 IAnalyzer::FArrayReader::GetSizeAndType() const
{
	const auto* Inner = (const FAuxData*)this;
	return Inner->FieldSizeAndType;
}
#endif // UE_TRACE_ANALYSIS_DEBUG_API

////////////////////////////////////////////////////////////////////////////////
const void* IAnalyzer::FArrayReader::GetImpl(uint32 Index, int8& SizeAndType) const
{
	const auto* Inner = (const FAuxData*)this;
	SizeAndType = Inner->FieldSizeAndType;
	uint32 Count = Num();
	if (Index >= Count)
	{
		return nullptr;
	}
	return (SizeAndType < 0) ?
		Inner->Data + (Index * (-SizeAndType)) :
		Inner->Data + (Index * SizeAndType);
}



// {{{1 event-data -------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
struct FEventDataInfo
{
	const FAuxData*		GetAuxData(uint32 FieldIndex) const;
	const uint8*		Ptr;
	const FDispatch&	Dispatch;
	FAuxDataCollector*	AuxCollector;
	uint16				Size;
};

////////////////////////////////////////////////////////////////////////////////
const FAuxData* FEventDataInfo::GetAuxData(uint32 FieldIndex) const
{
	const auto* Info = (const FEventDataInfo*)this;
	if (Info->AuxCollector == nullptr)
	{
		return nullptr;
	}

	for (FAuxData& Data : *(Info->AuxCollector))
	{
		if (Data.FieldIndex == FieldIndex)
		{
			if (Data.bFragmented)
			{
				Info->AuxCollector->Defragment(Data);
			}

			const FDispatch::FField& Field = Info->Dispatch.Fields[FieldIndex];
			Data.FieldSizeAndType = Field.SizeAndType;
			Data.bSigned = (Field.Class & UE::Trace::Protocol0::Field_Signed) != 0;
			return &Data;
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
const IAnalyzer::FEventTypeInfo& IAnalyzer::FEventData::GetTypeInfo() const
{
	const auto* Info = (const FEventDataInfo*)this;
	return (const FEventTypeInfo&)(Info->Dispatch);
}

////////////////////////////////////////////////////////////////////////////////
const IAnalyzer::FArrayReader* IAnalyzer::FEventData::GetArrayImpl(const ANSICHAR* FieldName) const
{
	const auto* Info = (const FEventDataInfo*)this;

	int32 Index = Info->Dispatch.GetFieldIndex(FieldName);
	if (Index >= 0)
	{
		if (const FAuxData* Data = Info->GetAuxData(Index))
		{
			return (IAnalyzer::FArrayReader*)Data;
		}
	}

	static const FAuxData EmptyAuxData = {};
	return (const IAnalyzer::FArrayReader*)&EmptyAuxData;
}

////////////////////////////////////////////////////////////////////////////////
const void* IAnalyzer::FEventData::GetValueImpl(const ANSICHAR* FieldName, int8& SizeAndType) const
{
	const auto* Info = (const FEventDataInfo*)this;
	const auto& Dispatch = Info->Dispatch;

	int32 Index = Dispatch.GetFieldIndex(FieldName);
	if (Index < 0)
	{
		return nullptr;
	}

	const auto& Field = Dispatch.Fields[Index];
	SizeAndType = Field.SizeAndType;
	return (Info->Ptr + Field.Offset);
}

////////////////////////////////////////////////////////////////////////////////
bool IAnalyzer::FEventData::GetString(const ANSICHAR* FieldName, FAnsiStringView& Out) const
{
	const auto* Info = (const FEventDataInfo*)this;
	const auto& Dispatch = Info->Dispatch;

	int32 Index = Dispatch.GetFieldIndex(FieldName);
	if (Index < 0)
	{
		return false;
	}

	const auto& Field = Dispatch.Fields[Index];
	if (Field.Class != UE::Trace::Protocol0::Field_String || Field.SizeAndType != sizeof(ANSICHAR))
	{
		return false;
	}

	if (const FAuxData* Data = Info->GetAuxData(Index))
	{
		Out = FAnsiStringView((const ANSICHAR*)(Data->Data), Data->DataSize);
		return true;
	}

	Out = FAnsiStringView();
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool IAnalyzer::FEventData::GetString(const ANSICHAR* FieldName, FWideStringView& Out) const
{
	const auto* Info = (const FEventDataInfo*)this;
	const auto& Dispatch = Info->Dispatch;

	int32 Index = Dispatch.GetFieldIndex(FieldName);
	if (Index < 0)
	{
		return false;
	}

	const auto& Field = Dispatch.Fields[Index];
	if (Field.Class != UE::Trace::Protocol0::Field_String || Field.SizeAndType != sizeof(WIDECHAR))
	{
		return false;
	}

	if (const FAuxData* Data = Info->GetAuxData(Index))
	{
		Out = FWideStringView((const WIDECHAR*)(Data->Data), Data->DataSize / sizeof(WIDECHAR));
		return true;
	}

	Out = FWideStringView();
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool IAnalyzer::FEventData::GetString(const ANSICHAR* FieldName, FString& Out) const
{
	const auto* Info = (const FEventDataInfo*)this;
	const auto& Dispatch = Info->Dispatch;

	int32 Index = Dispatch.GetFieldIndex(FieldName);
	if (Index < 0)
	{
		return false;
	}

	const auto& Field = Dispatch.Fields[Index];
	if (Field.Class == UE::Trace::Protocol0::Field_String)
	{
		const FAuxData* Data = Info->GetAuxData(Index);
		if (Data == nullptr)
		{
			Out = FString();
			return true;
		}

		if (Field.SizeAndType == sizeof(ANSICHAR))
		{
			Out = FString::ConstructFromPtrSize((const ANSICHAR*)(Data->Data), Data->DataSize);
			return true;
		}

		if (Field.SizeAndType == sizeof(WIDECHAR))
		{
			Out = FWideStringView((const WIDECHAR*)(Data->Data), Data->DataSize / sizeof(WIDECHAR));
			return true;
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
uint32 IAnalyzer::FEventData::GetSize() const
{
	const auto* Info = (const FEventDataInfo*)this;
	uint32 Size = Info->Size;
	if (Info->AuxCollector)
	{
		for (const FAuxData& Data : *(Info->AuxCollector))
		{
			Size += Data.DataSize;
		}
	}
	return Size;
}

////////////////////////////////////////////////////////////////////////////////
void IAnalyzer::FEventData::SerializeToCbor(TArray<uint8>& Out) const
{
	const auto* Info = (const FEventDataInfo*)this;
	uint32 Size = Info->Size;
	if (Info->AuxCollector != nullptr)
	{
		for (FAuxData& Data : *(Info->AuxCollector))
		{
			Size += Data.DataSize;
		}
	}
	SerializeToCborImpl(Out, *this, Size);
}

////////////////////////////////////////////////////////////////////////////////
const void* IAnalyzer::FEventData::GetValueRaw(FEventFieldHandle Handle) const
{
	const auto* Info = (const FEventDataInfo*)this;
	if (!Handle.IsValid())
	{
		return nullptr;
	}
	return Info->Ptr + Handle.Detail;
}

////////////////////////////////////////////////////////////////////////////////
const uint8* IAnalyzer::FEventData::GetAttachment() const
{
	const auto* Info = (const FEventDataInfo*)this;
	return Info->Ptr + Info->Dispatch.EventSize;
}

////////////////////////////////////////////////////////////////////////////////
uint32 IAnalyzer::FEventData::GetAttachmentSize() const
{
	const auto* Info = (const FEventDataInfo*)this;
	return Info->Size - Info->Dispatch.EventSize;
}

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ANALYSIS_DEBUG_API
const uint8* IAnalyzer::FEventData::GetRawPointer() const
{
	const auto* Info = (const FEventDataInfo*)this;
	return Info->Ptr;
}
#endif // UE_TRACE_ANALYSIS_DEBUG_API

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ANALYSIS_DEBUG_API
uint32 IAnalyzer::FEventData::GetRawSize() const
{
	const auto* Info = (const FEventDataInfo*)this;
	return Info->Size;
}
#endif // UE_TRACE_ANALYSIS_DEBUG_API

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ANALYSIS_DEBUG_API
uint32 IAnalyzer::FEventData::GetAuxSize() const
{
	const auto* Info = (const FEventDataInfo*)this;

	if (Info->Dispatch.MaybeHasAux())
	{
		uint32 Size = 0;
		if (Info->AuxCollector)
		{
			for (FAuxData& Data : *(Info->AuxCollector))
			{
				Size += 4; // aux field header
				Size += Data.DataSize;
			}
		}
		++Size; // aux terminator
		return Size;
	}
	else
	{
		return 0;
	}
}
#endif // UE_TRACE_ANALYSIS_DEBUG_API

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ANALYSIS_DEBUG_API
uint32 IAnalyzer::FEventData::GetTotalSize(IAnalyzer::EStyle Style, const IAnalyzer::FOnEventContext& Context, uint32 ProtocolVersion) const
{
	uint32 Size = 0;

	if (Style == EStyle::EnterScope)
	{
		// Include the size of the previous EnterScope event.
		if (Context.EventTime.GetTimestamp() == 0)
		{
			Size = 1; // uid
		}
		else
		{
			Size = 8; // 1 uid + 7 timestamp
		}
	}
	else if (Style == EStyle::LeaveScope)
	{
		// Return only the size of the LeaveScope event.
		if (Context.EventTime.GetTimestamp() == 0)
		{
			return 1; // uid
		}
		else
		{
			return 8; // 1 uid + 7 timestamp
		}
	}

	const auto* Info = (const FEventDataInfo*)this;

	const uint16 KnownEventUids = (ProtocolVersion >= 5) ? Protocol5::EKnownEventUids::User :
								  (ProtocolVersion == 4) ? Protocol4::EKnownEventUids::User : 0;
	static_assert(Protocol6::EKnownEventUids::User == Protocol5::EKnownEventUids::User, "Protocol6::EKnownEventUids::User");
	static_assert(Protocol7::EKnownEventUids::User == Protocol5::EKnownEventUids::User, "Protocol7::EKnownEventUids::User");

	// Add header size.
	if (Info->Dispatch.Uid < KnownEventUids)
	{
		Size += 1; // uid
	}
	else if (Info->Dispatch.IsImportant())
	{
		Size += 4; // sizeof(UE::Trace::Protocol5::FImportantEventHeader);
	}
	else if (Info->Dispatch.IsNoSync())
	{
		if (ProtocolVersion < 5)
		{
			Size += 4; // sizeof(UE::Trace::Protocol2::FEventHeader);
		}
		else
		{
			Size += 2; // uid == sizeof(UE::Trace::Protocol5::FEventHeader);
		}
	}
	else // Sync
	{
		if (ProtocolVersion < 5)
		{
			Size += 7; // sizeof(UE::Trace::Protocol2::FEventHeaderSync);
		}
		else
		{
			Size += 2 + 3; // uid + serial == sizeof(UE::Trace::Protocol5::FEventHeaderSync);
		}
	}

	// Add fixed size and attachment size.
	Size += Info->Size;

	// Add aux data size (including headers).
	Size += GetAuxSize();

	return Size;
}
#endif // UE_TRACE_ANALYSIS_DEBUG_API

////////////////////////////////////////////////////////////////////////////////
bool IAnalyzer::FEventData::IsDefinitionImpl(uint32& OutTypeId) const
{
	const auto* Info = (const FEventDataInfo*)this;
	OutTypeId = Info->Dispatch.Uid;
	return Info->Dispatch.IsDefinition();
}

////////////////////////////////////////////////////////////////////////////////
const void* IAnalyzer::FEventData::GetReferenceValueImpl(const char* FieldName, uint16& OutSizeType, uint32& OutTypeUid) const
{
	const auto* Info = (const FEventDataInfo*)this;
	const auto& Dispatch = Info->Dispatch;

	const int32 Index = Dispatch.GetFieldIndex(FieldName);
	if (Index < 0)
	{
		return nullptr;
	}

	const auto& Field = Dispatch.Fields[Index];
	if (Field.RefUid)
	{
		OutSizeType = Field.SizeAndType;
		OutTypeUid = Field.RefUid;
		return (Info->Ptr + Field.Offset);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
const void* IAnalyzer::FEventData::GetReferenceValueImpl(uint32 FieldIndex, uint32& OutTypeUid) const
{
	const auto* Info = (const FEventDataInfo*)this;
	const auto& Dispatch = Info->Dispatch;
	const auto& Field = Dispatch.Fields[FieldIndex];
	if (Field.RefUid)
	{
		OutTypeUid = Field.RefUid;
		return (Info->Ptr + Field.Offset);
	}
	return nullptr;
}

// }}}



// {{{1 type-registry ----------------------------------------------------------
class FTypeRegistry
{
public:
	typedef FDispatch FTypeInfo;

						~FTypeRegistry();
	const FTypeInfo*	Add(const void* TraceData, uint32 Version);
	const FTypeInfo*	AddVersion4(const void* TraceData);
	const FTypeInfo*	AddVersion6(const void* TraceData);
	void				Add(FTypeInfo* TypeInfo);
	const FTypeInfo*	Get(uint32 Uid) const;
	bool				IsUidValid(uint32 Uid) const;

private:
	TArray<FTypeInfo*>	TypeInfos;
};

////////////////////////////////////////////////////////////////////////////////
FTypeRegistry::~FTypeRegistry()
{
	for (FTypeInfo* TypeInfo : TypeInfos)
	{
		FMemory::Free(TypeInfo);
	}
}

////////////////////////////////////////////////////////////////////////////////
const FTypeRegistry::FTypeInfo* FTypeRegistry::Add(const void* TraceData, uint32 Version)
{
	switch(Version)
	{
	case 0: return AddVersion4(TraceData);
	case 4: return AddVersion4(TraceData);
	case 6: return AddVersion6(TraceData);
	case 7: return AddVersion6(TraceData);
	default:
		check(false); // Unknown version
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
const FTypeRegistry::FTypeInfo* FTypeRegistry::Get(uint32 Uid) const
{
	if (Uid >= uint32(TypeInfos.Num()))
	{
		return nullptr;
	}

	return TypeInfos[Uid];
}

////////////////////////////////////////////////////////////////////////////////
const FTypeRegistry::FTypeInfo* FTypeRegistry::AddVersion4(const void* TraceData)
{
	FDispatchBuilder Builder;

	const auto& NewEvent = *(Protocol4::FNewEventEvent*)(TraceData);

	const auto* NameCursor = (const ANSICHAR*)(NewEvent.Fields + NewEvent.FieldCount);

	Builder.SetLoggerName(NameCursor, NewEvent.LoggerNameSize);
	NameCursor += NewEvent.LoggerNameSize;

	Builder.SetEventName(NameCursor, NewEvent.EventNameSize);
	NameCursor += NewEvent.EventNameSize;
	Builder.SetUid(NewEvent.EventUid);

	// Fill out the fields
	for (int32 i = 0, n = NewEvent.FieldCount; i < n; ++i)
	{
		const auto& Field = NewEvent.Fields[i];

		int8 TypeSize = (int8)(1 << (Field.TypeInfo & Protocol0::Field_Pow2SizeMask));
		if (Field.TypeInfo & Protocol0::Field_Float)
		{
			TypeSize = static_cast<int8>(-TypeSize);
		}

		auto& OutField = Builder.AddField(NameCursor, Field.NameSize, Field.Size);
		OutField.Offset = Field.Offset;
		OutField.SizeAndType = TypeSize;

		OutField.Class = Field.TypeInfo & Protocol0::Field_SpecialMask;
		OutField.bArray = !!(Field.TypeInfo & Protocol0::Field_Array);

		NameCursor += Field.NameSize;
	}

	// NewEvent.Flags contains Protocol1::EEventFlags,
	// so it can be used in place for IAnalyzer::EEventFlags.
	static_assert((uint8)UE::Trace::Protocol1::EEventFlags::Important == (uint8)IAnalyzer::EEventFlags::Important, "");
	static_assert((uint8)UE::Trace::Protocol1::EEventFlags::MaybeHasAux == (uint8)IAnalyzer::EEventFlags::MaybeHasAux, "");
	static_assert((uint8)UE::Trace::Protocol1::EEventFlags::NoSync == (uint8)IAnalyzer::EEventFlags::NoSync, "");
	Builder.SetFlags(static_cast<IAnalyzer::EEventFlags>(NewEvent.Flags));

	FTypeInfo* TypeInfo = Builder.Finalize();
	Add(TypeInfo);

	return TypeInfo;
}

////////////////////////////////////////////////////////////////////////////////
const FTypeRegistry::FTypeInfo* FTypeRegistry::AddVersion6(const void* TraceData)
{
	using namespace Protocol6;
	FDispatchBuilder Builder;

	const auto& NewEvent = *(FNewEventEvent*)(TraceData);

	const auto* NameCursor = (const ANSICHAR*)(NewEvent.Fields + NewEvent.FieldCount);

	Builder.SetLoggerName(NameCursor, NewEvent.LoggerNameSize);
	NameCursor += NewEvent.LoggerNameSize;

	Builder.SetEventName(NameCursor, NewEvent.EventNameSize);
	NameCursor += NewEvent.EventNameSize;
	Builder.SetUid(NewEvent.EventUid);

	// Fill out the fields
	for (int32 i = 0, n = NewEvent.FieldCount; i < n; ++i)
	{
		const auto& Field = NewEvent.Fields[i];
		if (Field.FieldType == EFieldFamily::Regular)
		{
			int8 TypeSize = (int8)(1 << (Field.Regular.TypeInfo & Protocol0::Field_Pow2SizeMask));
			if (Field.Regular.TypeInfo & Protocol0::Field_Float)
			{
				TypeSize = static_cast<int8>(-TypeSize);
			}

			auto& OutField = Builder.AddField(NameCursor, Field.Regular.NameSize, Field.Regular.Size);
			OutField.Offset = Field.Regular.Offset;
			OutField.SizeAndType = TypeSize;

			OutField.Class = Field.Regular.TypeInfo & Protocol0::Field_SpecialMask;
			OutField.bArray = !!(Field.Regular.TypeInfo & Protocol0::Field_Array);

			NameCursor += Field.Regular.NameSize;
		}
		else if (Field.FieldType == EFieldFamily::Reference)
		{
			check((Field.Reference.TypeInfo & Protocol0::Field_CategoryMask) == Protocol0::Field_Integer);
			const int8 TypeSize = (int8)(1 << (Field.Reference.TypeInfo & Protocol0::Field_Pow2SizeMask));

			auto& OutField = Builder.AddField(NameCursor, Field.Reference.NameSize, TypeSize);
			OutField.Offset = Field.Reference.Offset;
			OutField.SizeAndType = TypeSize;
			OutField.RefUid = uint32(Field.Reference.RefUid);
			OutField.Class = 0;
			OutField.bArray = false;

			NameCursor += Field.Reference.NameSize;
		}
		else if (Field.FieldType == EFieldFamily::DefinitionId)
		{
			check((Field.DefinitionId.TypeInfo & Protocol0::Field_CategoryMask) == Protocol0::Field_Integer);
			const int8 TypeSize = (int8)(1 << (Field.DefinitionId.TypeInfo & Protocol0::Field_Pow2SizeMask));

			auto DefinitionIdFieldName = ANSITEXTVIEW("DefinitionId");
			auto& OutField = Builder.AddField(DefinitionIdFieldName.GetData(), DefinitionIdFieldName.Len(), TypeSize);
			OutField.Offset = Field.DefinitionId.Offset;
			OutField.SizeAndType = TypeSize;
			OutField.RefUid = NewEvent.EventUid;
			OutField.Class = 0;
			OutField.bArray = false;
		}
		else
		{
			// Error!
			continue;
		}
	}

	// NewEvent.Flags contains Protocol6::EEventFlags,
	// so it can be used in place for IAnalyzer::EEventFlags.
	static_assert((uint8)UE::Trace::Protocol6::EEventFlags::None == (uint8)IAnalyzer::EEventFlags::None, "");
	static_assert((uint8)UE::Trace::Protocol6::EEventFlags::Important == (uint8)IAnalyzer::EEventFlags::Important, "");
	static_assert((uint8)UE::Trace::Protocol6::EEventFlags::MaybeHasAux == (uint8)IAnalyzer::EEventFlags::MaybeHasAux, "");
	static_assert((uint8)UE::Trace::Protocol6::EEventFlags::NoSync == (uint8)IAnalyzer::EEventFlags::NoSync, "");
	static_assert((uint8)UE::Trace::Protocol6::EEventFlags::Definition == (uint8)IAnalyzer::EEventFlags::Definition, "");
	Builder.SetFlags(static_cast<IAnalyzer::EEventFlags>(NewEvent.Flags));

	FTypeInfo* TypeInfo = Builder.Finalize();
	Add(TypeInfo);

	return TypeInfo;
}

////////////////////////////////////////////////////////////////////////////////
void FTypeRegistry::Add(FTypeInfo* TypeInfo)
{
	// Add the type to the type-infos table. Usually duplicates are an error
	// but due to backwards compatibility we'll override existing types.
	uint16 Uid = TypeInfo->Uid;
	if (Uid < uint32(TypeInfos.Num()))
	{
		if (TypeInfos[Uid] != nullptr)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Warning: Override type for Uid=%u", uint32(Uid));
			FMemory::Free(TypeInfos[Uid]);
			TypeInfos[Uid] = nullptr;
		}
	}
	else
	{
#if UE_TRACE_ANALYSIS_DEBUG
		if (Uid > TypeInfos.Num() + 100)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Warning: Unexpected large Uid=%u", uint32(Uid));
		}
#endif // UE_TRACE_ANALYSIS_DEBUG
		TypeInfos.SetNum(Uid + 1);
	}

	TypeInfos[Uid] = TypeInfo;
}

////////////////////////////////////////////////////////////////////////////////
bool FTypeRegistry::IsUidValid(uint32 Uid) const
{
	return (Uid < uint32(TypeInfos.Num())) && (TypeInfos[Uid] != nullptr);
}

// {{{1 analyzer-hub -----------------------------------------------------------
class FAnalyzerHub
{
public:
	void				End();
	void				SetAnalyzers(TArray<IAnalyzer*>&& InAnalyzers);
	uint32				GetActiveAnalyzerNum() const;
	void				OnNewType(const FTypeRegistry::FTypeInfo* TypeInfo);
	void				OnEvent(const FTypeRegistry::FTypeInfo& TypeInfo, IAnalyzer::EStyle Style, const IAnalyzer::FOnEventContext& Context);
	void				OnThreadInfo(const FThreads::FInfo& ThreadInfo);
	void				OnVersion(uint32 TransportVersion, uint32 ProtocolVersion);

private:
	void				BuildRoutes();
	void				AddRoute(uint16 AnalyzerIndex, uint16 Id, const ANSICHAR* Logger, const ANSICHAR* Event, bool bScoped);
	int32				GetRouteIndex(const FTypeRegistry::FTypeInfo& TypeInfo);
	void				RetireAnalyzer(IAnalyzer* Analyzer);
	template <typename ImplType>
	void				ForEachRoute(uint32 RouteIndex, bool bScoped, ImplType&& Impl) const;

	struct FRoute
	{
		uint32			Hash;
		uint16			AnalyzerIndex : 15;
		uint16			bScoped : 1;
		uint16			Id;
		union
		{
			uint32		ParentHash;
			struct
			{
				int16	Count;
				int16	Parent;
			};
		};
	};

	typedef TArray<uint16, TInlineAllocator<96>> TypeToRouteArray;

	TArray<IAnalyzer*>	Analyzers;
	TArray<FRoute>		Routes;
	TypeToRouteArray	TypeToRoute; // biases by one so zero represents no route
};

////////////////////////////////////////////////////////////////////////////////
void FAnalyzerHub::End()
{
	for (IAnalyzer* Analyzer : Analyzers)
	{
		if (Analyzer != nullptr)
		{
			Analyzer->OnAnalysisEnd();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalyzerHub::SetAnalyzers(TArray<IAnalyzer*>&& InAnalyzers)
{
	Analyzers = MoveTemp(InAnalyzers);
	BuildRoutes();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FAnalyzerHub::GetActiveAnalyzerNum() const
{
	uint32 Count = 0;
	for (IAnalyzer* Analyzer : Analyzers)
	{
		Count += (Analyzer != nullptr);
	}
	return Count;
}

////////////////////////////////////////////////////////////////////////////////
int32 FAnalyzerHub::GetRouteIndex(const FTypeRegistry::FTypeInfo& TypeInfo)
{
	if (TypeInfo.Uid >= uint32(TypeToRoute.Num()))
	{
		return -1;
	}

	return int32(TypeToRoute[TypeInfo.Uid]) - 1;
}

////////////////////////////////////////////////////////////////////////////////
template <typename ImplType>
void FAnalyzerHub::ForEachRoute(uint32 RouteIndex, bool bScoped, ImplType&& Impl) const
{
	uint32 RouteCount = Routes.Num();
	if (RouteIndex >= RouteCount)
	{
		return;
	}

	const FRoute* FirstRoute = Routes.GetData();
	const FRoute* Route = FirstRoute + RouteIndex;
	do
	{
		const FRoute* NextRoute = FirstRoute + Route->Parent;
		for (uint32 n = Route->Count; n--; ++Route)
		{
			if (Route->bScoped != (bScoped == true))
			{
				continue;
			}

			IAnalyzer* Analyzer = Analyzers[Route->AnalyzerIndex];
			if (Analyzer != nullptr)
			{
				Impl(Analyzer, Route->Id);
			}
		}
		Route = NextRoute;
	}
	while (Route >= FirstRoute);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalyzerHub::OnNewType(const FTypeRegistry::FTypeInfo* TypeInfo)
{
	// Find routes that have subscribed to this event.
	auto FindRoute = [this] (uint32 Hash)
	{
		int32 Index = Algo::LowerBoundBy(Routes, Hash, [] (const FRoute& Route) { return Route.Hash; });
		return (Index < Routes.Num() && Routes[Index].Hash == Hash) ? Index : -1;
	};

	int32 FirstRoute = FindRoute(TypeInfo->Hash);
	if (FirstRoute < 0)
	{
		FFnv1aHash LoggerHash;
		LoggerHash.Add((const ANSICHAR*)TypeInfo + TypeInfo->LoggerNameOffset);
		if ((FirstRoute = FindRoute(LoggerHash.Get())) < 0)
		{
			FirstRoute = FindRoute(FFnv1aHash().Get());
		}
	}

	uint16 Uid = TypeInfo->Uid;
	if (Uid >= uint16(TypeToRoute.Num()))
	{
		TypeToRoute.SetNumZeroed(Uid + 32);
	}

	TypeToRoute[Uid] = uint16(FirstRoute + 1);

#if UE_TRACE_ANALYSIS_DEBUG
	const IAnalyzer::FEventTypeInfo& EventTypeInfo = *(IAnalyzer::FEventTypeInfo*)TypeInfo;
	UE_TRACE_ANALYSIS_DEBUG_LOG("NewEvent: Uid=%u (%s.%s) Flags=%s%s%s%s",
		EventTypeInfo.GetId(), EventTypeInfo.GetLoggerName(), EventTypeInfo.GetName(),
		EventTypeInfo.IsDefinition() ? "Definition|" : "",
		EventTypeInfo.IsImportant() ? "Important|" : "",
		EventTypeInfo.MaybeHasAux() ? "MaybeHasAux|" : "",
		EventTypeInfo.IsNoSync() ? "NoSync" : "Sync");
#endif // UE_TRACE_ANALYSIS_DEBUG

	// Inform routes that a new event has been declared.
	if (FirstRoute >= 0)
	{
		ForEachRoute(FirstRoute, false, [&] (IAnalyzer* Analyzer, uint16 RouteId)
		{
			if (!Analyzer->OnNewEvent(RouteId, *(IAnalyzer::FEventTypeInfo*)TypeInfo))
			{
				RetireAnalyzer(Analyzer);
			}
		});
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalyzerHub::OnEvent(
	const FTypeRegistry::FTypeInfo& TypeInfo,
	const IAnalyzer::EStyle Style,
	const IAnalyzer::FOnEventContext& Context)
{
	int32 RouteIndex = GetRouteIndex(TypeInfo);
	if (RouteIndex < 0)
	{
		return;
	}

	bool bScoped = (Style != IAnalyzer::EStyle::Normal);
	ForEachRoute(RouteIndex, bScoped, [&] (IAnalyzer* Analyzer, uint16 RouteId)
	{
		if (!Analyzer->OnEvent(RouteId, Style, Context))
		{
			RetireAnalyzer(Analyzer);
		}
	});
}

////////////////////////////////////////////////////////////////////////////////
void FAnalyzerHub::OnThreadInfo(const FThreads::FInfo& ThreadInfo)
{
	const auto& OuterThreadInfo = (IAnalyzer::FThreadInfo&)ThreadInfo;
	for (IAnalyzer* Analyzer : Analyzers)
	{
		if (Analyzer != nullptr)
		{
			Analyzer->OnThreadInfo(OuterThreadInfo);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalyzerHub::OnVersion(uint32 TransportVersion, uint32 ProtocolVersion)
{
	for (IAnalyzer* Analyzer : Analyzers)
	{
		if (Analyzer != nullptr)
		{
			Analyzer->OnVersion(TransportVersion, ProtocolVersion);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalyzerHub::BuildRoutes()
{
	// Call out to all registered analyzers to have them register event interest
	struct : public IAnalyzer::FInterfaceBuilder
	{
		virtual void RouteEvent(uint16 RouteId, const ANSICHAR* Logger, const ANSICHAR* Event, bool bScoped) override
		{
			Self->AddRoute(AnalyzerIndex, RouteId, Logger, Event, bScoped);
		}

		virtual void RouteLoggerEvents(uint16 RouteId, const ANSICHAR* Logger, bool bScoped) override
		{
			Self->AddRoute(AnalyzerIndex, RouteId, Logger, "", bScoped);
		}

		virtual void RouteAllEvents(uint16 RouteId, bool bScoped) override
		{
			Self->AddRoute(AnalyzerIndex, RouteId, "", "", bScoped);
		}

		FAnalyzerHub* Self;
		uint16 AnalyzerIndex;
	} Builder;
	Builder.Self = this;

	IAnalyzer::FOnAnalysisContext OnAnalysisContext = { Builder };
	for (uint16 i = 0, n = uint16(Analyzers.Num()); i < n; ++i)
	{
		uint32 RouteCount = Routes.Num();

		Builder.AnalyzerIndex = i;
		IAnalyzer* Analyzer = Analyzers[i];
		Analyzer->OnAnalysisBegin(OnAnalysisContext);

		// If the analyzer didn't add any routes we'll retire it immediately
		if (RouteCount == Routes.Num())
		{
			RetireAnalyzer(Analyzer);
		}
	}

	// Sort routes by their hashes.
	auto RouteProjection = [] (const FRoute& Route) { return Route.Hash; };
	Algo::SortBy(Routes, RouteProjection);

	auto FindRoute = [this, &RouteProjection] (uint32 Hash)
	{
		int32 Index = Algo::LowerBoundBy(Routes, Hash, RouteProjection);
		return (Index < Routes.Num() && Routes[Index].Hash == Hash) ? Index : -1;
	};

	int32 AllEventsIndex = FindRoute(FFnv1aHash().Get());
	auto FixupRoute = [this, &FindRoute, AllEventsIndex] (FRoute* Route)
	{
		if (Route->ParentHash)
		{
			int32 ParentIndex = FindRoute(Route->ParentHash);
			Route->Parent = int16((ParentIndex < 0) ? AllEventsIndex : ParentIndex);
		}
		else
		{
			Route->Parent = -1;
		}
		Route->Count = 1;
		return Route;
	};

	FRoute* Cursor = FixupRoute(Routes.GetData());
	for (uint16 i = 1, n = uint16(Routes.Num()); i < n; ++i)
	{
		FRoute* Route = Routes.GetData() + i;
		if (Route->Hash == Cursor->Hash)
		{
			Cursor->Count++;
		}
		else
		{
			Cursor = FixupRoute(Route);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalyzerHub::RetireAnalyzer(IAnalyzer* Analyzer)
{
	for (uint32 i = 0, n = Analyzers.Num(); i < n; ++i)
	{
		if (Analyzers[i] != Analyzer)
		{
			continue;
		}

		Analyzer->OnAnalysisEnd();
		Analyzers[i] = nullptr;
		break;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalyzerHub::AddRoute(
	uint16 AnalyzerIndex,
	uint16 Id,
	const ANSICHAR* Logger,
	const ANSICHAR* Event,
	bool bScoped)
{
	check(AnalyzerIndex < Analyzers.Num());

	uint32 ParentHash = 0;
	if (Logger[0] && Event[0])
	{
		FFnv1aHash Hash;
		Hash.Add(Logger);
		ParentHash = Hash.Get();
	}

	FFnv1aHash Hash;
	Hash.Add(Logger);
	Hash.Add(Event);

	FRoute& Route = Routes.Emplace_GetRef();
	Route.Id = Id;
	Route.Hash = Hash.Get();
	Route.ParentHash = ParentHash;
	Route.AnalyzerIndex = AnalyzerIndex;
	Route.bScoped = (bScoped == true);
}

// {{{1 state ------------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
struct FAnalysisState
{
	struct FSerial
	{
		uint32	Value = 0;
		uint32	Mask = 0;
	};

	FThreads	Threads;
	FTiming		Timing;
	FSerial		Serial;
	uint32		UserUidBias = Protocol4::EKnownEventUids::User;

#if UE_TRACE_ANALYSIS_DEBUG
	uint32 TempUidBytes = 0; // for protocol 4
	int32 MaxEventDescs = 0; // for protocol 5
	int32 SerialWrappedCount = 0; // for protocol 5
	int32 NumSkippedSerialGaps = 0; // for protocol 5
	uint64 NumSkippedSerials = 0; // for protocol 5

	uint64 TotalEventCount = 0;
	uint64 NewEventCount = 0;
	uint64 SyncEventCount = 0;
	uint64 ImportantNoSyncEventCount = 0;
	uint64 OtherNoSyncEventCount = 0;
	uint64 EnterScopeEventCount = 0;
	uint64 LeaveScopeEventCount = 0;
	uint64 EnterScopeTEventCount = 0;
	uint64 LeaveScopeTEventCount = 0;

	uint64 TotalEventSize = 0;
	uint64 NewEventSize = 0;
	uint64 SyncEventSize = 0;
	uint64 ImportantNoSyncEventSize = 0;
	uint64 OtherNoSyncEventSize = 0;
	uint64 EnterScopeEventSize = 0;
	uint64 LeaveScopeEventSize = 0;
	uint64 EnterScopeTEventSize = 0;
	uint64 LeaveScopeTEventSize = 0;
#endif // UE_TRACE_ANALYSIS_DEBUG
};



// {{{1 thread-info-cb ---------------------------------------------------------
class FThreadInfoCallback
{
public:
	virtual			~FThreadInfoCallback() {}
	virtual void	OnThreadInfo(const FThreads::FInfo& Info) = 0;
};



// {{{1 analyzer ---------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
enum
{
	RouteId_NewTrace,
	RouteId_SerialSync,
	RouteId_Timing,
	RouteId_ThreadTiming,
	RouteId_ThreadInfo,
	RouteId_ThreadGroupBegin,
	RouteId_ThreadGroupEnd,
};

////////////////////////////////////////////////////////////////////////////////
class FTraceAnalyzer
	: public IAnalyzer
{
public:
							FTraceAnalyzer(FAnalysisState& InState, FThreadInfoCallback& InCallback);
	virtual bool			OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;
	virtual void			OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	void					OnNewTrace(const FOnEventContext& Context);
	void					OnSerialSync(const FOnEventContext& Context);
	void					OnThreadTiming(const FOnEventContext& Context);
	void					OnThreadInfoInternal(const FOnEventContext& Context);
	void					OnThreadGroupBegin(const FOnEventContext& Context);
	void					OnThreadGroupEnd();
	void					OnTiming(const FOnEventContext& Context);

private:
	FAnalysisState&			State;
	FThreadInfoCallback&	ThreadInfoCallback;
};

////////////////////////////////////////////////////////////////////////////////
FTraceAnalyzer::FTraceAnalyzer(FAnalysisState& InState, FThreadInfoCallback& InCallback)
: State(InState)
, ThreadInfoCallback(InCallback)
{
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;
	Builder.RouteEvent(RouteId_NewTrace,		"$Trace", "NewTrace");
	Builder.RouteEvent(RouteId_SerialSync,		"$Trace", "SerialSync");
	Builder.RouteEvent(RouteId_Timing,			"$Trace", "Timing");
	Builder.RouteEvent(RouteId_ThreadTiming,	"$Trace", "ThreadTiming");
	Builder.RouteEvent(RouteId_ThreadInfo,		"$Trace", "ThreadInfo");
	Builder.RouteEvent(RouteId_ThreadGroupBegin,"$Trace", "ThreadGroupBegin");
	Builder.RouteEvent(RouteId_ThreadGroupEnd,	"$Trace", "ThreadGroupEnd");
}

////////////////////////////////////////////////////////////////////////////////
bool FTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	switch (RouteId)
	{
	case RouteId_NewTrace:			OnNewTrace(Context);			break;
	case RouteId_SerialSync:		OnSerialSync(Context);			break;
	case RouteId_Timing:			OnTiming(Context);				break;
	case RouteId_ThreadTiming:		OnThreadTiming(Context);		break;
	case RouteId_ThreadInfo:		OnThreadInfoInternal(Context);	break;
	case RouteId_ThreadGroupBegin:	OnThreadGroupBegin(Context);	break;
	case RouteId_ThreadGroupEnd:	OnThreadGroupEnd();				break;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAnalyzer::OnNewTrace(const FOnEventContext& Context)
{
	const FEventData& EventData = Context.EventData;

	// "Serial" will tell us approximately where we've started in the log serial
	// range. We'll bias it by half so we won't accept any serialized events and
	// mark the MSB to indicate that the current serial should be corrected.
	auto& Serial = State.Serial;
	uint32 Hint = EventData.GetValue<uint32>("Serial");
	UE_TRACE_ANALYSIS_DEBUG_LOG("Serial=%u (hint)", Hint);
	Hint -= (Serial.Mask + 1) >> 1;
	Hint &= Serial.Mask;
	Serial.Value = Hint;
	Serial.Value |= 0xc0000000;
	UE_TRACE_ANALYSIS_DEBUG_LOG("Serial.Value=%u|0xC0000000 (Next=%u)", Hint, Serial.Value & Serial.Mask);
	UE_TRACE_ANALYSIS_DEBUG_LOG("Serial.Mask=0x%X", Serial.Mask);

	// Later traces will have an explicit "SerialSync" trace event to indicate
	// when there is enough data to establish the correct log serial
	const uint8 FeatureSet = EventData.GetValue<uint8>("FeatureSet");
	UE_TRACE_ANALYSIS_DEBUG_LOG("FeatureSet=%d", FeatureSet);
	if ((FeatureSet & 1) == 0)
	{
		OnSerialSync(Context);
	}

	State.UserUidBias = EventData.GetValue<uint32>("UserUidBias", uint32(UE::Trace::Protocol3::EKnownEventUids::User));
	UE_TRACE_ANALYSIS_DEBUG_LOG("UserUidBias=%u", State.UserUidBias);

	OnTiming(Context);
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAnalyzer::OnSerialSync(const FOnEventContext& Context)
{
	State.Serial.Value &= ~0x40000000;
	UE_TRACE_ANALYSIS_DEBUG_LOG("SerialSync: Next=%u", State.Serial.Value & State.Serial.Mask);
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAnalyzer::OnTiming(const FOnEventContext& Context)
{
	uint64 StartCycle = Context.EventData.GetValue<uint64>("StartCycle");
	uint64 CycleFrequency = Context.EventData.GetValue<uint64>("CycleFrequency");
	double StartDateTime = Context.EventData.GetValue<double>("StartDateTime");

	State.Timing.BaseTimestamp = StartCycle;
	State.Timing.TimestampHz = CycleFrequency;
	State.Timing.InvTimestampHz = 1.0 / double(CycleFrequency);
	State.Timing.BaseDateTimestamp = StartDateTime;
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAnalyzer::OnThreadTiming(const FOnEventContext& Context)
{
	uint64 BaseTimestamp = Context.EventData.GetValue<uint64>("BaseTimestamp");
	if (FThreads::FInfo* Info = State.Threads.GetInfo())
	{
		Info->PrevTimestamp = BaseTimestamp;

		// We can springboard of this event as a way to know a thread has just
		// started (or at least is about to send its first event). Notify analyzers
		// so they're aware of threads that never get explicitly registered.
		ThreadInfoCallback.OnThreadInfo(*Info);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAnalyzer::OnThreadInfoInternal(const FOnEventContext& Context)
{
	const FEventData& EventData = Context.EventData;

	FThreads::FInfo* ThreadInfo;
	uint32 ThreadId = EventData.GetValue<uint32>("ThreadId", ~0u);
	if (ThreadId != ~0u)
	{
		// Post important-events; the thread-info event is not on the thread it
		// represents anymore. Fortunately the thread-id is traced now.
		ThreadInfo = State.Threads.GetInfo(ThreadId);
	}
	else
	{
		ThreadInfo = State.Threads.GetInfo();
	}

	if (ThreadInfo == nullptr)
	{
		return;
	}

	ThreadInfo->SystemId = EventData.GetValue<uint32>("SystemId");
	ThreadInfo->SortHint = EventData.GetValue<int32>("SortHint");

	FAnsiStringView Name;
	EventData.GetString("Name", Name);
	ThreadInfo->Name.SetNumUninitialized(Name.Len() + 1);
	ThreadInfo->Name[Name.Len()] = '\0';
	FMemory::Memcpy(ThreadInfo->Name.GetData(), Name.GetData(), Name.Len());

	if (ThreadInfo->GroupName.Num() <= 0)
	{
		if (const TArray<uint8>* GroupName = State.Threads.GetGroupName())
		{
			ThreadInfo->GroupName = *GroupName;
		}
	}

	UE_TRACE_ANALYSIS_DEBUG_LOG("Thread: Tid=%u SysId=0x%X Name=\"%s\"", ThreadInfo->ThreadId, ThreadInfo->SystemId, ThreadInfo->Name.GetData());

	ThreadInfoCallback.OnThreadInfo(*ThreadInfo);
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAnalyzer::OnThreadGroupBegin(const FOnEventContext& Context)
{
	const FEventData& EventData = Context.EventData;

	FAnsiStringView Name;
	EventData.GetString("Name", Name);
	State.Threads.SetGroupName(Name.GetData(), Name.Len());
}

////////////////////////////////////////////////////////////////////////////////
void FTraceAnalyzer::OnThreadGroupEnd()
{
	State.Threads.SetGroupName("", 0);
}



// {{{1 bridge -----------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FAnalysisBridge
	: public FThreadInfoCallback
{
public:
	typedef FAnalysisState::FSerial FSerial;

						FAnalysisBridge(TArray<IAnalyzer*>&& Analyzers);
	bool				IsStillAnalyzing() const;
	void				Reset();
	FAnalysisState&		GetState();
	uint32				GetUserUidBias() const;
	FSerial&			GetSerial();
	uint32				GetActiveThreadId() const;
	void				SetActiveThread(uint32 ThreadId);
	void				OnNewType(const FTypeRegistry::FTypeInfo* TypeInfo);
	void				OnEvent(const FEventDataInfo& EventDataInfo);
	virtual void		OnThreadInfo(const FThreads::FInfo& InThreadInfo) override;
	void				EnterScope();
	void				EnterScope(uint64 RelativeTimestamp);
	void				EnterScopeA(uint64 AbsoluteTimestamp);
	void				EnterScopeB(uint64 BaseRelativeTimestamp);
	void				LeaveScope();
	void				LeaveScope(uint64 RelativeTimestamp);
	void				LeaveScopeA(uint64 AbsoluteTimestamp);
	void				LeaveScopeB(uint64 BaseRelativeTimestamp);
	void				OnVersion(uint32 TransportVersion, uint32 ProtocolVersion) { AnalyzerHub.OnVersion(TransportVersion, ProtocolVersion); }
#if UE_TRACE_ANALYSIS_DEBUG
	bool				ShouldWriteDebugLog();
	void				DebugLogEvent(const FTypeRegistry::FTypeInfo* TypeInfo, uint32 FixedSize, uint32 AuxSize, uint32 Serial);
	void				DebugLogNewEvent(uint32 Uid, const FTypeRegistry::FTypeInfo* TypeInfo, uint32 EventSize);
	void				DebugLogEnterScopeEvent(uint32 Uid, uint32 EventSize);
	void				DebugLogEnterScopeEvent(uint32 Uid, uint64 RelativeTimestamp, uint32 EventSize);
	void				DebugLogEnterScopeAEvent(uint32 Uid, uint64 AbsoluteTimestamp, uint32 EventSize);
	void				DebugLogEnterScopeBEvent(uint32 Uid, uint64 BaseRelativeTimestamp, uint32 EventSize);
	void				DebugLogLeaveScopeEvent(uint32 Uid, uint32 EventSize);
	void				DebugLogLeaveScopeEvent(uint32 Uid, uint64 RelativeTimestamp, uint32 EventSize);
	void				DebugLogLeaveScopeAEvent(uint32 Uid, uint64 AbsoluteTimestamp, uint32 EventSize);
	void				DebugLogLeaveScopeBEvent(uint32 Uid, uint64 BaseRelativeTimestamp, uint32 EventSize);
#endif // UE_TRACE_ANALYSIS_DEBUG

private:
	void				DispatchLeaveScope();
	FAnalysisState		State;
	FTraceAnalyzer		TraceAnalyzer = { State, *this };
	FAnalyzerHub		AnalyzerHub;
	FThreads::FInfo*	ThreadInfo = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
FAnalysisBridge::FAnalysisBridge(TArray<IAnalyzer*>&& Analyzers)
{
	TArray<IAnalyzer*> TempAnalyzers(MoveTemp(Analyzers));
	TempAnalyzers.Add(&TraceAnalyzer);
	AnalyzerHub.SetAnalyzers(MoveTemp(TempAnalyzers));
}

////////////////////////////////////////////////////////////////////////////////
bool FAnalysisBridge::IsStillAnalyzing() const
{
	// "> 1" because TraceAnalyzer is always present but shouldn't be considered
	return AnalyzerHub.GetActiveAnalyzerNum() > 1;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::Reset()
{
	AnalyzerHub.End();

	State.~FAnalysisState();
	new (&State) FAnalysisState();
}

////////////////////////////////////////////////////////////////////////////////
FAnalysisState& FAnalysisBridge::GetState()
{
	return State;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FAnalysisBridge::GetUserUidBias() const
{
	return State.UserUidBias;
}

////////////////////////////////////////////////////////////////////////////////
FAnalysisBridge::FSerial& FAnalysisBridge::GetSerial()
{
	return State.Serial;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FAnalysisBridge::GetActiveThreadId() const
{
	return ThreadInfo ? ThreadInfo->ThreadId : ~0u;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::SetActiveThread(uint32 ThreadId)
{
	ThreadInfo = State.Threads.GetInfo(ThreadId);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::OnNewType(const FTypeRegistry::FTypeInfo* TypeInfo)
{
	AnalyzerHub.OnNewType(TypeInfo);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::OnEvent(const FEventDataInfo& EventDataInfo)
{
	// TODO "Dispatch" should be renamed "EventTypeInfo" or similar.
	const FTypeRegistry::FTypeInfo* TypeInfo = &(EventDataInfo.Dispatch);

	IAnalyzer::EStyle Style = IAnalyzer::EStyle::Normal;
	if (ThreadInfo->ScopeRoutes.Num() > 0 && int64(ThreadInfo->ScopeRoutes.Last()) < 0)
	{
		Style = IAnalyzer::EStyle::EnterScope;
		State.Timing.EventTimestamp = ~(ThreadInfo->ScopeRoutes.Last());
		ThreadInfo->ScopeRoutes.Last() = PTRINT(TypeInfo);
	}
	else
	{
		State.Timing.EventTimestamp = 0;
	}

	IAnalyzer::FOnEventContext Context = {
		*(const IAnalyzer::FThreadInfo*)ThreadInfo,
		(const IAnalyzer::FEventTime&)(State.Timing),
		(const IAnalyzer::FEventData&)EventDataInfo,
	};
	AnalyzerHub.OnEvent(*TypeInfo, Style, Context);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::OnThreadInfo(const FThreads::FInfo& InThreadInfo)
{
	// Note that InThreadInfo might not equal the bridge's ThreadInfo because
	// information about threads comes from trace and could have been traced on
	// a different thread to the one it is describing (or no thread at all in
	// the case of important events).
	AnalyzerHub.OnThreadInfo(InThreadInfo);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::EnterScope()
{
	ThreadInfo->ScopeRoutes.Push(~0);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::EnterScope(uint64 Timestamp)
{
	Timestamp = ThreadInfo->PrevTimestamp += Timestamp;
	ThreadInfo->ScopeRoutes.Push(~Timestamp);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::EnterScopeA(uint64 Timestamp)
{
	Timestamp -= State.Timing.BaseTimestamp;
	ThreadInfo->ScopeRoutes.Push(~Timestamp);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::EnterScopeB(uint64 Timestamp)
{
	ThreadInfo->ScopeRoutes.Push(~Timestamp);
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::LeaveScope()
{
	State.Timing.EventTimestamp = 0;
	DispatchLeaveScope();
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::LeaveScope(uint64 Timestamp)
{
	Timestamp = ThreadInfo->PrevTimestamp += Timestamp;
	State.Timing.EventTimestamp = Timestamp;
	DispatchLeaveScope();
	State.Timing.EventTimestamp = 0;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::LeaveScopeA(uint64 Timestamp)
{
	Timestamp -= State.Timing.BaseTimestamp;
	State.Timing.EventTimestamp = Timestamp;
	DispatchLeaveScope();
	State.Timing.EventTimestamp = 0;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::LeaveScopeB(uint64 Timestamp)
{
	State.Timing.EventTimestamp = Timestamp;
	DispatchLeaveScope();
	State.Timing.EventTimestamp = 0;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DispatchLeaveScope()
{
	if (ThreadInfo->ScopeRoutes.Num() <= 0)
	{
		// Leave scope without a corresponding enter
		return;
	}

	int64 ScopeValue = int64(ThreadInfo->ScopeRoutes.Pop(EAllowShrinking::No));
	if (ScopeValue < 0)
	{
		// enter/leave pair without an event in-between.
		return;
	}

	const auto* TypeInfo = (FTypeRegistry::FTypeInfo*)PTRINT(ScopeValue);

	FEventDataInfo EmptyEventInfo = {
		nullptr,
		*TypeInfo
	};

	IAnalyzer::FOnEventContext Context = {
		*(const IAnalyzer::FThreadInfo*)ThreadInfo,
		(const IAnalyzer::FEventTime&)(State.Timing),
		(const IAnalyzer::FEventData&)EmptyEventInfo,
	};

	AnalyzerHub.OnEvent(*TypeInfo, IAnalyzer::EStyle::LeaveScope, Context);
}

#if UE_TRACE_ANALYSIS_DEBUG
////////////////////////////////////////////////////////////////////////////////
bool FAnalysisBridge::ShouldWriteDebugLog()
{
	// Example for limiting debug logs to first 100K events...
	//return (State.TotalEventCount < 100000);
	return UE::Trace::Analysis::GDebugLevel >= 2;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogEvent(const FTypeRegistry::FTypeInfo* TypeInfo, uint32 FixedSize, uint32 AuxSize, uint32 Serial)
{
	++State.TotalEventCount;
	uint32 EventSize = FixedSize + AuxSize;
	State.TotalEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder();
		UE_TRACE_ANALYSIS_DEBUG_Appendf("[EVENT %llu]", State.TotalEventCount);

		if (GetActiveThreadId() != ~0u)
		{
			UE_TRACE_ANALYSIS_DEBUG_Appendf(" Tid=%u", GetActiveThreadId());
		}

		UE_TRACE_ANALYSIS_DEBUG_Appendf(" Uid=%u", uint32(TypeInfo->Uid));

		const IAnalyzer::FEventTypeInfo& EventTypeInfo = *(IAnalyzer::FEventTypeInfo*)TypeInfo;
		UE_TRACE_ANALYSIS_DEBUG_Appendf(" (%s.%s)", EventTypeInfo.GetLoggerName(), EventTypeInfo.GetName());

		UE_TRACE_ANALYSIS_DEBUG_Appendf(" Size=%u", EventSize);
		if (AuxSize != 0)
		{
			UE_TRACE_ANALYSIS_DEBUG_Appendf(" (%u + %u)", FixedSize, AuxSize);
		}
	}

	if (TypeInfo->IsSync())
	{
		++State.SyncEventCount;
		State.SyncEventSize += EventSize;

		if (bDebugLog)
		{
			UE_TRACE_ANALYSIS_DEBUG_Appendf(" Sync=%u", Serial);
		}
	}
	else if (TypeInfo->IsImportant())
	{
		++State.ImportantNoSyncEventCount;
		State.ImportantNoSyncEventSize += EventSize;

		if (bDebugLog)
		{
			UE_TRACE_ANALYSIS_DEBUG_Append(" Important");
		}
	}
	else
	{
		++State.OtherNoSyncEventCount;
		State.OtherNoSyncEventSize += EventSize;
	}

	if (bDebugLog)
	{
		UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder();
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogNewEvent(uint32 Uid, const FTypeRegistry::FTypeInfo* TypeInfo, uint32 EventSize)
{
	++State.TotalEventCount;
	++State.NewEventCount;
	State.TotalEventSize += EventSize;
	State.NewEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		const uint32 Tid = ETransportTid::Importants; // GetActiveThreadId()
		UE_TRACE_ANALYSIS_DEBUG_LOG("[EVENT %llu] Tid=%u Uid=%u (NewEvent) Size=%u", State.TotalEventCount, Tid, Uid, EventSize);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogEnterScopeEvent(uint32 Uid, uint32 EventSize)
{
	++State.TotalEventCount;
	++State.EnterScopeEventCount;
	State.TotalEventSize += EventSize;
	State.EnterScopeEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("[EVENT %llu] Tid=%u Uid=%u (EnterScope) Size=%u",
			State.TotalEventCount, GetActiveThreadId(), Uid, EventSize);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogEnterScopeEvent(uint32 Uid, uint64 RelativeTimestamp, uint32 EventSize)
{
	++State.TotalEventCount;
	++State.EnterScopeTEventCount;
	State.TotalEventSize += EventSize;
	State.EnterScopeTEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		check(ThreadInfo);
		UE_TRACE_ANALYSIS_DEBUG_LOG("[EVENT %llu] Tid=%u Uid=%u (EnterScope_T) Timestamp=(+%llu)=%llu Size=%u",
			State.TotalEventCount, GetActiveThreadId(), Uid,
			RelativeTimestamp, State.Timing.BaseTimestamp + ThreadInfo->PrevTimestamp + RelativeTimestamp,
			EventSize);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogEnterScopeAEvent(uint32 Uid, uint64 AbsoluteTimestamp, uint32 EventSize)
{
	++State.TotalEventCount;
	++State.EnterScopeTEventCount;
	State.TotalEventSize += EventSize;
	State.EnterScopeTEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("[EVENT %llu] Tid=%u Uid=%u (EnterScope_TA) Timestamp=%llu Size=%u",
			State.TotalEventCount, GetActiveThreadId(), Uid,
			AbsoluteTimestamp,
			EventSize);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogEnterScopeBEvent(uint32 Uid, uint64 BaseAbsoluteTimestamp, uint32 EventSize)
{
	++State.TotalEventCount;
	++State.EnterScopeTEventCount;
	State.TotalEventSize += EventSize;
	State.EnterScopeTEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("[EVENT %llu] Tid=%u Uid=%u (EnterScope_TB) Timestamp=(+%llu)=%llu Size=%u",
			State.TotalEventCount, GetActiveThreadId(), Uid,
			BaseAbsoluteTimestamp, State.Timing.BaseTimestamp + BaseAbsoluteTimestamp,
			EventSize);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogLeaveScopeEvent(uint32 Uid, uint32 EventSize)
{
	++State.TotalEventCount;
	++State.LeaveScopeEventCount;
	State.TotalEventSize += EventSize;
	State.LeaveScopeEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("[EVENT %llu] Tid=%u Uid=%u (LeaveScope) Size=%u",
			State.TotalEventCount, GetActiveThreadId(), Uid, EventSize);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogLeaveScopeEvent(uint32 Uid, uint64 RelativeTimestamp, uint32 EventSize)
{
	++State.TotalEventCount;
	++State.LeaveScopeTEventCount;
	State.TotalEventSize += EventSize;
	State.LeaveScopeTEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		check(ThreadInfo);
		UE_TRACE_ANALYSIS_DEBUG_LOG("[EVENT %llu] Tid=%u Uid=%u (LeaveScope_T) Timestamp=(+%llu)=%llu Size=%u",
			State.TotalEventCount, GetActiveThreadId(), Uid,
			RelativeTimestamp, State.Timing.BaseTimestamp + ThreadInfo->PrevTimestamp + RelativeTimestamp,
			EventSize);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogLeaveScopeAEvent(uint32 Uid, uint64 AbsoluteTimestamp, uint32 EventSize)
{
	++State.TotalEventCount;
	++State.LeaveScopeTEventCount;
	State.TotalEventSize += EventSize;
	State.LeaveScopeTEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("[EVENT %llu] Tid=%u Uid=%u (LeaveScope_TA) Timestamp=%llu Size=%u",
			State.TotalEventCount, GetActiveThreadId(), Uid,
			AbsoluteTimestamp,
			EventSize);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisBridge::DebugLogLeaveScopeBEvent(uint32 Uid, uint64 BaseAbsoluteTimestamp, uint32 EventSize)
{
	++State.TotalEventCount;
	++State.LeaveScopeTEventCount;
	State.TotalEventSize += EventSize;
	State.LeaveScopeTEventSize += EventSize;

	const bool bDebugLog = ShouldWriteDebugLog();
	if (bDebugLog)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("[EVENT %llu] Tid=%u Uid=%u (LeaveScope_TB) Timestamp=(+%llu)=%llu Size=%u",
			State.TotalEventCount, GetActiveThreadId(), Uid,
			BaseAbsoluteTimestamp, State.Timing.BaseTimestamp + BaseAbsoluteTimestamp,
			EventSize);
	}
}
#endif // UE_TRACE_ANALYSIS_DEBUG



// {{{1 machine ----------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FAnalysisMachine
{
public:
	enum class EStatus
	{
		Error,
		Abort,
		NotEnoughData,
		EndOfStream,
		Continue,
		Sync,
	};

	struct FMachineContext
	{
		FAnalysisMachine&	Machine;
		FAnalysisBridge&	Bridge;
		FMessageDelegate&	OnMessage;

		inline void EmitMessage(EAnalysisMessageSeverity Severity, FStringView Message) const
		{
			const bool _ = OnMessage.ExecuteIfBound(Severity, Message);
		}

		template<typename FormatType, typename... Types>
		inline void EmitMessagef(EAnalysisMessageSeverity Severity, const FormatType& Format, Types... Args) const
		{
			TStringBuilder<128> FormattedMessage;
			FormattedMessage.Appendf(Format, Forward<Types>(Args)...);
			EmitMessage(Severity, FormattedMessage.ToView());
		}
	};

	class FStage
	{
	public:
		typedef FAnalysisMachine::FMachineContext	FMachineContext;
		typedef FAnalysisMachine::EStatus			EStatus;

		virtual				~FStage() {}
		virtual EStatus		OnData(FStreamReader& Reader, const FMachineContext& Context) = 0;
		virtual void		EnterStage(const FMachineContext& Context) {};
		virtual void		ExitStage(const FMachineContext& Context) {};
	};

							FAnalysisMachine(FAnalysisBridge& InBridge, FMessageDelegate&& InMessage);
							~FAnalysisMachine();
	EStatus					OnData(FStreamReader& Reader);
	void					Transition();
	template <class StageType, typename... ArgsType>
	StageType*				QueueStage(ArgsType... Args);

private:
	void					CleanUp();
	FAnalysisBridge&		Bridge;
	FStage*					ActiveStage = nullptr;
	TArray<FStage*>			StageQueue;
	TArray<FStage*>			DeadStages;
	FMessageDelegate		OnMessage;
};

////////////////////////////////////////////////////////////////////////////////
FAnalysisMachine::FAnalysisMachine(FAnalysisBridge& InBridge, FMessageDelegate&& InMessage)
: Bridge(InBridge)
, OnMessage(InMessage)
{
}

////////////////////////////////////////////////////////////////////////////////
FAnalysisMachine::~FAnalysisMachine()
{
	CleanUp();
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisMachine::CleanUp()
{
	for (FStage* Stage : DeadStages)
	{
		delete Stage;
	}
	DeadStages.Reset();
}

////////////////////////////////////////////////////////////////////////////////
template <class StageType, typename... ArgsType>
StageType* FAnalysisMachine::QueueStage(ArgsType... Args)
{
	StageType* Stage = new StageType(Args...);
	StageQueue.Insert(Stage, 0);
	return Stage;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisMachine::Transition()
{
	if (ActiveStage != nullptr)
	{
		const FMachineContext Context = { *this, Bridge, OnMessage };
		ActiveStage->ExitStage(Context);

		DeadStages.Add(ActiveStage);
	}

	ActiveStage = (StageQueue.Num() > 0) ? StageQueue.Pop() : nullptr;

	if (ActiveStage != nullptr)
	{
		const FMachineContext Context = { *this, Bridge, OnMessage };
		ActiveStage->EnterStage(Context);
	}
}

////////////////////////////////////////////////////////////////////////////////
FAnalysisMachine::EStatus FAnalysisMachine::OnData(FStreamReader& Reader)
{
	const FMachineContext Context = { *this, Bridge, OnMessage };
	EStatus Ret;
	do
	{
		CleanUp();
		check(ActiveStage != nullptr);
		Ret = ActiveStage->OnData(Reader, Context);
	}
	while (Ret == EStatus::Continue);
	return Ret;
}



// {{{1 protocol-0 -------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FProtocol0Stage
	: public FAnalysisMachine::FStage
{
public:
						FProtocol0Stage(FTransport* InTransport);
	virtual				~FProtocol0Stage();

	virtual EStatus		OnData(FStreamReader& Reader, const FMachineContext& Context) override;
	virtual void		EnterStage(const FMachineContext& Context) override;
	virtual void		ExitStage(const FMachineContext& Context) override;

private:
	FTypeRegistry		TypeRegistry;
	FTransport*			Transport;
};

////////////////////////////////////////////////////////////////////////////////
FProtocol0Stage::FProtocol0Stage(FTransport* InTransport)
: Transport(InTransport)
{
}

////////////////////////////////////////////////////////////////////////////////
FProtocol0Stage::~FProtocol0Stage()
{
	delete Transport;
}

////////////////////////////////////////////////////////////////////////////////
void FProtocol0Stage::EnterStage(const FMachineContext& Context)
{
	Context.Bridge.SetActiveThread(0);
}

////////////////////////////////////////////////////////////////////////////////
void FProtocol0Stage::ExitStage(const FMachineContext& Context)
{
	// Ensure the transport does not have pending buffers (i.e. event data not yet processed).
	if (!Transport->IsEmpty())
	{
		Context.EmitMessage(EAnalysisMessageSeverity::Warning, TEXT("Transport buffers are not empty at end of analysis (protocol 0)!"));
	}

#if UE_TRACE_ANALYSIS_DEBUG
	Transport->DebugEnd();
#endif
}

////////////////////////////////////////////////////////////////////////////////
FProtocol0Stage::EStatus FProtocol0Stage::OnData(
	FStreamReader& Reader,
	const FMachineContext& Context)
{
	Transport->SetReader(Reader);

	while (true)
	{
		const auto* Header = Transport->GetPointer<Protocol0::FEventHeader>();
		if (Header == nullptr)
		{
			break;
		}

		uint32 BlockSize = Header->Size + sizeof(Protocol0::FEventHeader);
		Header = Transport->GetPointer<Protocol0::FEventHeader>(BlockSize);
		if (Header == nullptr)
		{
			break;
		}

		uint32 Uid = uint32(Header->Uid) & uint32(Protocol0::EKnownEventUids::UidMask);

		if (Uid == uint32(Protocol0::EKnownEventUids::NewEvent))
		{
			// There is no need to check size here as the runtime never builds
			// packets that fragment new-event events.
			const FTypeRegistry::FTypeInfo* TypeInfo = TypeRegistry.Add(Header->EventData, 0);
#if UE_TRACE_ANALYSIS_DEBUG
			Context.Bridge.DebugLogNewEvent(Uid, TypeInfo, BlockSize);
#endif
			Context.Bridge.OnNewType(TypeInfo);
			Transport->Advance(BlockSize);
			continue;
		}

		const FTypeRegistry::FTypeInfo* TypeInfo = TypeRegistry.Get(Uid);
		if (TypeInfo == nullptr)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Invalid TypeInfo for Uid %u (Tid=%u)", Uid, Context.Bridge.GetActiveThreadId());
			return EStatus::Error;
		}

#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogEvent(TypeInfo, BlockSize, 0, Context.Bridge.GetSerial().Value); //???
#endif

		FEventDataInfo EventDataInfo = {
			Header->EventData,
			*TypeInfo,
			nullptr,
			Header->Size
		};

		Context.Bridge.OnEvent(EventDataInfo);

		Transport->Advance(BlockSize);
	}

	return Reader.IsEmpty() ? EStatus::EndOfStream : EStatus::NotEnoughData;
}



// {{{1 protocol-2 -------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FProtocol2Stage
	: public FAnalysisMachine::FStage
{
public:
						FProtocol2Stage(uint32 Version, FTransport* InTransport);
	virtual				~FProtocol2Stage();

	virtual EStatus		OnData(FStreamReader& Reader, const FMachineContext& Context) override;
	virtual void		EnterStage(const FMachineContext& Context) override;
	virtual void		ExitStage(const FMachineContext& Context) override;

protected:
	virtual int32		OnData(FStreamReader& Reader, FAnalysisBridge& Bridge);
	int32				OnDataAux(FStreamReader& Reader, FAuxDataCollector& Collector);
	FTypeRegistry		TypeRegistry;
	FTransport*			Transport;
	uint32				ProtocolVersion;
	uint32				SerialInertia = ~0u;
};

////////////////////////////////////////////////////////////////////////////////
FProtocol2Stage::FProtocol2Stage(uint32 Version, FTransport* InTransport)
: Transport(InTransport)
, ProtocolVersion(Version)
{
	switch (ProtocolVersion)
	{
	case Protocol0::EProtocol::Id:
	case Protocol1::EProtocol::Id:
	case Protocol2::EProtocol::Id:
		{
			FDispatchBuilder Dispatch;
			Dispatch.SetUid(uint16(Protocol2::EKnownEventUids::NewEvent));
			Dispatch.SetLoggerName("$Trace");
			Dispatch.SetEventName("NewEvent");
			TypeRegistry.Add(Dispatch.Finalize());
		}
		break;

	case Protocol3::EProtocol::Id:
		{
			FDispatchBuilder Dispatch;
			Dispatch.SetUid(uint16(Protocol3::EKnownEventUids::NewEvent));
			Dispatch.SetLoggerName("$Trace");
			Dispatch.SetEventName("NewEvent");
			Dispatch.SetFlags(IAnalyzer::EEventFlags::NoSync);
			TypeRegistry.Add(Dispatch.Finalize());
		}
		break;
	}
}

////////////////////////////////////////////////////////////////////////////////
FProtocol2Stage::~FProtocol2Stage()
{
	delete Transport;
}


////////////////////////////////////////////////////////////////////////////////
void FProtocol2Stage::EnterStage(const FMachineContext& Context)
{
	auto& Serial = Context.Bridge.GetSerial();

	if (ProtocolVersion == Protocol1::EProtocol::Id)
	{
		Serial.Mask = 0x0000ffff;
	}
	else
	{
		Serial.Mask = 0x00ffffff;
	}
	UE_TRACE_ANALYSIS_DEBUG_LOG("Serial.Mask = 0x%X", Serial.Mask);
}

////////////////////////////////////////////////////////////////////////////////
void FProtocol2Stage::ExitStage(const FMachineContext& Context)
{
	// Ensure the transport does not have pending buffers (i.e. event data not yet processed).
	if (!Transport->IsEmpty())
	{
		Context.EmitMessagef(EAnalysisMessageSeverity::Warning, TEXT("Transport buffers are not empty at end of analysis (protocol %u)!"), ProtocolVersion);
	}

#if UE_TRACE_ANALYSIS_DEBUG
	Transport->DebugEnd();
#endif
}

////////////////////////////////////////////////////////////////////////////////

FProtocol2Stage::EStatus FProtocol2Stage::OnData(
	FStreamReader& Reader,
	const FMachineContext& Context)
{
	auto* InnerTransport = (FTidPacketTransport*)Transport;
	InnerTransport->SetReader(Reader);
	const FTidPacketTransport::ETransportResult Result = InnerTransport->Update();
	if (Result == FTidPacketTransport::ETransportResult::Error)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: An error was detected in the transport layer, most likely due to a corrupt trace file.");
		Context.EmitMessage(
			EAnalysisMessageSeverity::Error,
			TEXT("An error was detected in the transport layer, most likely due to a corrupt trace file. See log for details.")
		);
		return EStatus::Error;
	}

	struct FRotaItem
	{
		uint32				Serial;
		uint32				ThreadId;
		FStreamReader*		Reader;

		bool operator < (const FRotaItem& Rhs) const
		{
			int32 Delta = Rhs.Serial - Serial;
			int32 Wrapped = uint32(Delta + 0x007f'fffe) >= uint32(0x00ff'fffd);
			return (Wrapped ^ (Delta > 0)) != 0;
		}
	};
	TArray<FRotaItem> Rota;

	for (uint32 i = 0, n = InnerTransport->GetThreadCount(); i < n; ++i)
	{
		FStreamReader* ThreadReader = InnerTransport->GetThreadStream(i);
		uint32 ThreadId = InnerTransport->GetThreadId(i);
		Rota.Add({~0u, ThreadId, ThreadReader});
	}

	auto& Serial = Context.Bridge.GetSerial();

	while (true)
	{
		uint32 ActiveCount = uint32(Rota.Num());

		for (uint32 i = 0; i < ActiveCount;)
		{
			FRotaItem& RotaItem = Rota[i];

			if (int32(RotaItem.Serial) > int32(Serial.Value & Serial.Mask))
			{
				++i;
				continue;
			}

			Context.Bridge.SetActiveThread(RotaItem.ThreadId);

			uint32 AvailableSerial = OnData(*(RotaItem.Reader), Context.Bridge);
			if (int32(AvailableSerial) >= 0)
			{
				RotaItem.Serial = AvailableSerial;
				if (Rota[0].Serial > AvailableSerial)
				{
					Swap(Rota[0], RotaItem);
				}
				++i;
			}
			else
			{
				FRotaItem TempItem = RotaItem;
				TempItem.Serial = ~0u;

				for (uint32 j = i, m = ActiveCount - 1; j < m; ++j)
				{
					Rota[j] = Rota[j + 1];
				}

				Rota[ActiveCount - 1] = TempItem;
				--ActiveCount;
			}

			if (((Rota[0].Serial - Serial.Value) & Serial.Mask) == 0)
			{
				i = 0;
			}
		}

		if (ActiveCount < 1)
		{
			break;
		}

		{
			TArrayView<FRotaItem> ActiveRota(Rota.GetData(), ActiveCount);
			Algo::Sort(ActiveRota);
		}

		int32 MinLogSerial = Rota[0].Serial;
		if (ActiveCount > 1)
		{
			int32 MaxLogSerial = Rota[ActiveCount - 1].Serial;

			if ((uint32(MinLogSerial - Serial.Value) & Serial.Mask) == 0)
			{
				continue;
			}

			// If min/max are more than half the serial range apart consider them
			// as having wrapped.
			int32 HalfRange = int32(Serial.Mask >> 1);
			if ((MaxLogSerial - MinLogSerial) >= HalfRange)
			{
				for (uint32 i = 0; i < ActiveCount; ++i)
				{
					FRotaItem& RotaItem = Rota[i];
					if (int32(RotaItem.Serial) >= HalfRange)
					{
						MinLogSerial = RotaItem.Serial;
						break;
					}
				}
			}
		}

		// If the current serial has its MSB set we're currently in a mode trying
		// to derive the best starting serial.
		if (int32(Serial.Value) < int32(0xc0000000))
		{
			Serial.Value = (MinLogSerial & Serial.Mask);
			UE_TRACE_ANALYSIS_DEBUG_LOG("New Serial.Value: %u", Serial.Value);
			continue;
		}

		// If we didn't stumble across the next serialized event we have done all
		// we can for now.
		if ((uint32(MinLogSerial - Serial.Value) & Serial.Mask) != 0)
		{
			break;
		}
	}

	// Patch the serial value to try and recover from gaps. Note that the bits
	// used to wait for the serial-sync event are ignored as that event may never
	// be reached if leading events are synchronized. Some inertia is added as
	// the missing range of events can be in subsequent packets. (Maximum inertia
	// need only be the size of the tail / io-read-size).
	if (Rota.Num() > 0)
	{
		int32 LowestSerial = int32(Rota[0].Serial);
		if (LowestSerial >= 0)
		{
			enum {
				InertiaLen		= 0x20,
				InertiaStart	= 0x7f - InertiaLen,
				InertiaBase		= InertiaStart << 25,	// leaves bit 24 unset so
				InertiaInc		= 1 << 25,				// ~0u can never happen
			};
			if (SerialInertia == ~0u)
			{
				SerialInertia = LowestSerial + InertiaBase;
			}
			else
			{
				SerialInertia += InertiaInc;
				if (int32(SerialInertia) >= 0)
				{
					if (SerialInertia == LowestSerial)
					{
						SerialInertia = ~0u;
						Serial.Value = LowestSerial;
						UE_TRACE_ANALYSIS_DEBUG_LOG("New Serial.Value: %u (LowestSerial)", Serial.Value);
						return OnData(Reader, Context);
					}
					SerialInertia = ~0u;
				}
			}
		}
	}

	return Reader.IsEmpty() ? EStatus::EndOfStream : EStatus::NotEnoughData;
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol2Stage::OnData(FStreamReader& Reader, FAnalysisBridge& Bridge)
{
	auto& Serial = Bridge.GetSerial();

	while (true)
	{
		auto Mark = Reader.SaveMark();

		const auto* Header = Reader.GetPointer<Protocol2::FEventHeader>();
		if (Header == nullptr)
		{
			break;
		}

		uint32 Uid = uint32(Header->Uid) & uint32(Protocol2::EKnownEventUids::UidMask);

		const FTypeRegistry::FTypeInfo* TypeInfo = TypeRegistry.Get(Uid);
		if (TypeInfo == nullptr)
		{
			// Event-types may not to be discovered in Uid order.
			UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Invalid TypeInfo for Uid %u (Tid=%u)", Uid, Bridge.GetActiveThreadId());
			break;
		}

		uint32 BlockSize = Header->Size;

		// Make sure we consume events in the correct order
		if (TypeInfo->IsSync())
		{
			switch (ProtocolVersion)
			{
			case Protocol1::EProtocol::Id:
				{
					const auto* HeaderV1 = (Protocol1::FEventHeader*)Header;
					if (HeaderV1->Serial != (Serial.Value & Serial.Mask))
					{
						UE_TRACE_ANALYSIS_DEBUG_LOG_EX(2, "Tid=%u --> EventSerial=%u", Bridge.GetActiveThreadId(), HeaderV1->Serial);
						return HeaderV1->Serial;
					}
					BlockSize += sizeof(*HeaderV1);
				}
				break;

			case Protocol2::EProtocol::Id:
			case Protocol3::EProtocol::Id:
				{
					const auto* HeaderSync = (Protocol2::FEventHeaderSync*)Header;
					uint32 EventSerial = HeaderSync->SerialLow|(uint32(HeaderSync->SerialHigh) << 16);
					if (EventSerial != (Serial.Value & Serial.Mask))
					{
						UE_TRACE_ANALYSIS_DEBUG_LOG_EX(2, "Tid=%u --> EventSerial=%u", Bridge.GetActiveThreadId(), EventSerial);
						return EventSerial;
					}
					BlockSize += sizeof(*HeaderSync);
				}
				break;
			}
		}
		else
		{
			BlockSize += sizeof(*Header);
		}

		if (Reader.GetPointer(BlockSize) == nullptr)
		{
			break;
		}

		Reader.Advance(BlockSize);

#if UE_TRACE_ANALYSIS_DEBUG
		uint32 AuxSize = 0;
#endif
		FAuxDataCollector AuxCollector;
		if (TypeInfo->MaybeHasAux())
		{
#if UE_TRACE_ANALYSIS_DEBUG
			auto AuxMark = Reader.SaveMark();
#endif

			int AuxStatus = OnDataAux(Reader, AuxCollector);
			if (AuxStatus == 0)
			{
				Reader.RestoreMark(Mark);
				break;
			}

#if UE_TRACE_ANALYSIS_DEBUG
			AuxSize = uint32(UPTRINT(Reader.SaveMark()) - UPTRINT(AuxMark));
#endif
		}

		if (TypeInfo->IsSync())
		{
			Serial.Value += 1;
			Serial.Value &= 0x3fffffff; // Don't set MSBs. They have other uses
		}

		auto* EventData = (const uint8*)Header + BlockSize - Header->Size;
		if (Uid == uint32(Protocol2::EKnownEventUids::NewEvent))
		{
			// There is no need to check size here as the runtime never builds
			// packets that fragment new-event events.
			TypeInfo = TypeRegistry.Add(EventData, 0);
#if UE_TRACE_ANALYSIS_DEBUG
			Bridge.DebugLogNewEvent(Uid, TypeInfo, uint32(UPTRINT(Reader.SaveMark()) - UPTRINT(Mark)));
#endif
			Bridge.OnNewType(TypeInfo);
		}
		else
		{
#if UE_TRACE_ANALYSIS_DEBUG
			const uint32 EventSize = uint32(UPTRINT(Reader.SaveMark()) - UPTRINT(Mark));
			Bridge.DebugLogEvent(TypeInfo, EventSize - AuxSize, AuxSize, Serial.Value - 1);
#endif
			FEventDataInfo EventDataInfo = {
				EventData,
				*TypeInfo,
				&AuxCollector,
				Header->Size,
			};

			Bridge.OnEvent(EventDataInfo);
		}
	}

	return -1;
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol2Stage::OnDataAux(FStreamReader& Reader, FAuxDataCollector& Collector)
{
	while (true)
	{
		const uint8* NextByte = Reader.GetPointer<uint8>();
		if (NextByte == nullptr)
		{
			return 0;
		}

		// Is the following sequence a blob of auxiliary data or the null
		// terminator byte?
		if (NextByte[0] == 0)
		{
			Reader.Advance(1);
			return 1;
		}

		// Get header and the auxiliary blob's size
		const auto* Header = Reader.GetPointer<Protocol1::FAuxHeader>();
		if (Header == nullptr)
		{
			return 0;
		}

		// Check it exists
		uint32 BlockSize = (Header->Size >> 8) + sizeof(*Header);
		if (Reader.GetPointer(BlockSize) == nullptr)
		{
			return 0;
		}

		// Attach to event
		FAuxData AuxData = {};
		AuxData.Data = Header->Data;
		AuxData.DataSize = uint32(BlockSize - sizeof(*Header));
		AuxData.FieldIndex = uint16(Header->FieldIndex & Protocol1::FAuxHeader::FieldMask);
		Collector.Push(AuxData);

		Reader.Advance(BlockSize);
	}
}



// {{{1 protocol-4 -------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FProtocol4Stage
	: public FProtocol2Stage
{
public:
						FProtocol4Stage(uint32 Version, FTransport* InTransport);

private:
	virtual int32		OnData(FStreamReader& Reader, FAnalysisBridge& Bridge) override;
	int32				OnDataImpl(FStreamReader& Reader, FAnalysisBridge& Bridge);
	int32				OnDataKnown(uint32 Uid, FStreamReader& Reader, FAnalysisBridge& Bridge);
};

////////////////////////////////////////////////////////////////////////////////
FProtocol4Stage::FProtocol4Stage(uint32 Version, FTransport* InTransport)
: FProtocol2Stage(Version, InTransport)
{
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol4Stage::OnData(FStreamReader& Reader, FAnalysisBridge& Bridge)
{
	while (true)
	{
		if (int32 TriResult = OnDataImpl(Reader, Bridge))
		{
			return (TriResult < 0) ? ~TriResult : -1;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol4Stage::OnDataImpl(FStreamReader& Reader, FAnalysisBridge& Bridge)
{
	auto& Serial = Bridge.GetSerial();

	/* Returns 0 if an event was successfully processed, 1 if there's not enough
	 * data available, or ~AvailableLogSerial if the pending event is in the future */

	auto Mark = Reader.SaveMark();

	const auto* UidCursor = Reader.GetPointer<uint8>();
	if (UidCursor == nullptr)
	{
		return 1;
	}

	uint32 UidBytes = 1 + !!(*UidCursor & Protocol4::EKnownEventUids::Flag_TwoByteUid);
	if (UidBytes > 1 && Reader.GetPointer(UidBytes) == nullptr)
	{
		return 1;
	}

	uint32 Uid = ~0u;
	switch (UidBytes)
	{
		case 1:	Uid = *UidCursor;			break;
		case 2:	Uid = *(uint16*)UidCursor;	break;
	}
	Uid >>= Protocol4::EKnownEventUids::_UidShift;

	if (Uid < Bridge.GetUserUidBias())
	{
		Reader.Advance(UidBytes);
#if UE_TRACE_ANALYSIS_DEBUG
		FAnalysisState& State = Bridge.GetState();
		State.TempUidBytes = UidBytes;
#endif
		if (!OnDataKnown(Uid, Reader, Bridge))
		{
			Reader.RestoreMark(Mark);
			return 1;
		}
		return 0;
	}

	// Do we know about this event type yet?
	const FTypeRegistry::FTypeInfo* TypeInfo = TypeRegistry.Get(Uid);
	if (TypeInfo == nullptr)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Invalid TypeInfo for Uid %u (Tid=%u)", Uid, Bridge.GetActiveThreadId());
		return 1;
	}

	if (!ensure(UidBytes == 2)) // see Protocol4::FEventHeader
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Invalid trace stream: Tid=%u Uid=%u encoded on 1 byte (instead of 2 bytes).", Bridge.GetActiveThreadId(), Uid);
		return 1;
	}

	// Parse the header
	const auto* Header = Reader.GetPointer<Protocol4::FEventHeader>();
	if (Header == nullptr)
	{
		return 1;
	}

	uint32 BlockSize = Header->Size;

	// Make sure we consume events in the correct order
	if (TypeInfo->IsSync())
	{
		if (Reader.GetPointer<Protocol4::FEventHeaderSync>() == nullptr)
		{
			return 1;
		}

		const auto* HeaderSync = (Protocol4::FEventHeaderSync*)Header;
		uint32 EventSerial = HeaderSync->SerialLow|(uint32(HeaderSync->SerialHigh) << 16);
		if (EventSerial != (Serial.Value & Serial.Mask))
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG_EX(2, "Tid=%u --> EventSerial=%u", Bridge.GetActiveThreadId(), EventSerial);
			return ~EventSerial;
		}
		BlockSize += sizeof(*HeaderSync);
	}
	else
	{
		BlockSize += sizeof(*Header);
	}

	// Is all the event's data available?
	if (Reader.GetPointer(BlockSize) == nullptr)
	{
		return 1;
	}

	Reader.Advance(BlockSize);

#if UE_TRACE_ANALYSIS_DEBUG
	uint32 AuxSize = 0;
#endif

	// Collect auxiliary data
	FAuxDataCollector AuxCollector;
	if (TypeInfo->MaybeHasAux())
	{
		// Important events' size may include their array data so we need to backtrack
		auto NextMark = Reader.SaveMark();
		if (TypeInfo->IsImportant())
		{
			Reader.RestoreMark(Mark);
			Reader.Advance(sizeof(Protocol4::FEventHeader) + TypeInfo->EventSize);
		}

#if UE_TRACE_ANALYSIS_DEBUG
		auto AuxMark = Reader.SaveMark();
#endif

		int AuxStatus = OnDataAux(Reader, AuxCollector);
		if (AuxStatus == 0)
		{
			Reader.RestoreMark(Mark);
			return 1;
		}

#if UE_TRACE_ANALYSIS_DEBUG
		AuxSize = uint32(UPTRINT(Reader.SaveMark()) - UPTRINT(AuxMark));
#endif

		// User error could have resulted in less space being used that was
		// allocated for important events. So we can't assume that aux data
		// reading has read all the way up to the next event. So we use marks
		if (TypeInfo->IsImportant())
		{
			Reader.RestoreMark(NextMark);
		}
	}

#if UE_TRACE_ANALYSIS_DEBUG
	const uint32 EventSize = uint32(UPTRINT(Reader.SaveMark()) - UPTRINT(Mark));
	Bridge.DebugLogEvent(TypeInfo, EventSize - AuxSize, AuxSize, Serial.Value);
#endif

	// Maintain sync
	if (TypeInfo->IsSync())
	{
		Serial.Value += 1;
		Serial.Value &= 0x7fffffff; // don't set msb. that has other uses
	}

	// Sent the event to subscribed analyzers
	FEventDataInfo EventDataInfo = {
		(const uint8*)Header + BlockSize - Header->Size,
		*TypeInfo,
		&AuxCollector,
		Header->Size,
	};

	Bridge.OnEvent(EventDataInfo);

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol4Stage::OnDataKnown(
	uint32 Uid,
	FStreamReader& Reader,
	FAnalysisBridge& Bridge)
{
#if UE_TRACE_ANALYSIS_DEBUG
	FAnalysisState& State = Bridge.GetState();
#endif

	switch (Uid)
	{
	case Protocol4::EKnownEventUids::NewEvent:
		{
			const auto* Size = Reader.GetPointer<uint16>();
			if (!Size)
			{
				return 0;
			}
			const void* EventTypeAndData = Reader.GetPointer(sizeof(*Size) + *Size);
			if (!EventTypeAndData)
			{
				return 0;
			}
			const FTypeRegistry::FTypeInfo* TypeInfo = TypeRegistry.Add(Size + 1, 4);
#if UE_TRACE_ANALYSIS_DEBUG
			Bridge.DebugLogNewEvent(Uid, TypeInfo, uint32(State.TempUidBytes + sizeof(*Size) + *Size));
#endif
			Bridge.OnNewType(TypeInfo);
			Reader.Advance(sizeof(*Size) + *Size);
		}
		return 1;

	case Protocol4::EKnownEventUids::EnterScope:
#if UE_TRACE_ANALYSIS_DEBUG
		Bridge.DebugLogEnterScopeEvent(Uid, State.TempUidBytes);
#endif
		Bridge.EnterScope();
		return 1;

	case Protocol4::EKnownEventUids::LeaveScope:
#if UE_TRACE_ANALYSIS_DEBUG
		Bridge.DebugLogLeaveScopeEvent(Uid, State.TempUidBytes);
#endif
		Bridge.LeaveScope();
		return 1;

	case Protocol4::EKnownEventUids::EnterScope_T:
		{
			const uint8* Stamp = Reader.GetPointer(sizeof(uint64) - 1);
			if (!Stamp)
			{
				return 0;
			}

			const uint64 RelativeTimestamp = *(uint64*)(Stamp - 1) >> 8;
#if UE_TRACE_ANALYSIS_DEBUG
			Bridge.DebugLogEnterScopeEvent(Uid, RelativeTimestamp, uint32(State.TempUidBytes + sizeof(uint64) - 1));
#endif
			Bridge.EnterScope(RelativeTimestamp);

			Reader.Advance(sizeof(uint64) - 1);
		}
		return 1;

	case Protocol4::EKnownEventUids::LeaveScope_T:
		{
			const uint8* Stamp = Reader.GetPointer(sizeof(uint64) - 1);
			if (!Stamp)
			{
				return 0;
			}

			const uint64 RelativeTimestamp = *(uint64*)(Stamp - 1) >> 8;
#if UE_TRACE_ANALYSIS_DEBUG
			Bridge.DebugLogLeaveScopeEvent(Uid, RelativeTimestamp, uint32(State.TempUidBytes + sizeof(uint64) - 1));
#endif
			Bridge.LeaveScope(RelativeTimestamp);

			Reader.Advance(sizeof(uint64) - 1);
		}
		return 1;

	default:
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Cannot process known event %llu with Uid %u", State.TotalEventCount, Uid);
		return 0;
	};
}



// {{{1 protocol-5 -------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FProtocol5Stage
	: public FAnalysisMachine::FStage
{
public:
						FProtocol5Stage(FTransport* InTransport);
	virtual				~FProtocol5Stage();

	virtual void		ExitStage(const FMachineContext& Context) override;

protected:
	struct alignas(16) FEventDesc
	{
		union
		{
			struct
			{
				int32		Serial;
				uint16		Uid			: 14;
				uint16		bTwoByteUid	: 1;
				uint16		bHasAux		: 1;
				uint16		AuxKey;
			};
			uint64			Meta = 0;
		};
		union
		{
			uint32			GapLength;
			const uint8*	Data = nullptr;
		};
#if UE_TRACE_ANALYSIS_DEBUG
		uint32				EventSize = 0;
		uint32				AuxSize = 0;
		uint64				Reserved = 0;
#endif
	};
#if UE_TRACE_ANALYSIS_DEBUG
	static_assert(sizeof(FEventDesc) == 32, "");
#else
	static_assert(sizeof(FEventDesc) == 16, "");
#endif

	struct alignas(16) FEventDescStream
	{
		uint32					ThreadId;
		uint32					TransportIndex;
		union
		{
			uint32				ContainerIndex;
			const FEventDesc*	EventDescs;
		};

		enum { GapThreadId = ~0u };
	};
	static_assert(sizeof(FEventDescStream) == 16, "");

	struct FSerialDistancePredicate
	{
		bool operator () (const FEventDescStream& Lhs, const FEventDescStream& Rhs) const
		{
			// Provided that less than approximately "SerialRange * BytesPerSerial"
			// is buffered there should never be more that "SerialRange / 2" serial
			// numbers. Thus if the distance between any two serial numbers is larger
			// than half the serial space, they have wrapped.
			uint32 Ld = Lhs.EventDescs->Serial - Origin;
			uint32 Rd = Rhs.EventDescs->Serial - Origin;
			return Ld < Rd;
		};
		uint32 Origin;
	};

	enum ESerial : int32
	{
		Bits		= 24,
		Mask		= (1 << Bits) - 1,
		Range		= 1 << Bits,
		Ignored		= Range << 2, // far away so proper serials always compare less-than
		Terminal,
	};

	using EventDescArray	= TArray<FEventDesc>;
	using EKnownUids		= Protocol5::EKnownEventUids;

	virtual EStatus			OnData(FStreamReader& Reader, const FMachineContext& Context) override;
	EStatus					OnDataNewEvents(const FMachineContext& Context);
	EStatus					OnDataImportant(const FMachineContext& Context);
	EStatus					OnDataPseudoImportant(const FMachineContext& Context);
	EStatus					OnDataNormal(const FMachineContext& Context);
	int32					ParseImportantEvents(FStreamReader& Reader, EventDescArray& OutEventDescs, const FMachineContext& Context);
	int32					ParseEvents(FStreamReader& Reader, EventDescArray& OutEventDescs, const FMachineContext& Context);
	int32					ParseEventsWithAux(FStreamReader& Reader, EventDescArray& OutEventDescs, const FMachineContext& Context);
	int32					ParseEvent(FStreamReader& Reader, FEventDesc& OutEventDesc, const FMachineContext& Context);
	virtual void			SetSizeIfKnownEvent(uint32 Uid, uint32& InOutEventSize);
	virtual bool			DispatchKnownEvent(const FMachineContext& Context, uint32 Uid, const FEventDesc* Cursor);
	int32					DispatchNormalEvents(const FMachineContext& Context, TArray<FEventDescStream>& EventDescHeap);
	int32					DispatchEvents(const FMachineContext& Context, TArray<FEventDescStream>& EventDescHeap);
	int32					DispatchEvents(const FMachineContext& Context, const FEventDesc* EventDesc, uint32 Count);
	void					DetectSerialGaps(TArray<FEventDescStream>& EventDescHeap);
	template <typename Callback>
	void					ForEachSerialGap(const TArray<FEventDescStream>& EventDescHeap, Callback&& InCallback);
#if UE_TRACE_ANALYSIS_DEBUG
	void					PrintParsedEvent(int EventIndex, const FEventDesc& EventDesc, int32 Size);
#endif // UE_TRACE_ANALYSIS_DEBUG

	FTypeRegistry			TypeRegistry;
	FTransport*				BaseTransport;
	FTidPacketTransport&	Transport;
	EventDescArray			EventDescs;
	EventDescArray			SerialGaps;
	uint32					NextSerial = ~0u;
	uint32					OldNextSerial = ~0u;
	uint32					NextSerialWaitCount = 0;
	uint32					SyncCount;
	uint32					EventVersion = 4; //Protocol version 5 uses the event version from protocol 4
	bool					bSkipSerialError = false;
	bool					bSkipSerial = false;
};

////////////////////////////////////////////////////////////////////////////////
FProtocol5Stage::FProtocol5Stage(FTransport* InTransport)
: BaseTransport(InTransport)
, Transport(*(FTidPacketTransport*)InTransport)
, SyncCount(Transport.GetSyncCount())
{
	EventDescs.Reserve(8 << 10);
}

////////////////////////////////////////////////////////////////////////////////
FProtocol5Stage::~FProtocol5Stage()
{
	delete BaseTransport;
}

////////////////////////////////////////////////////////////////////////////////
void FProtocol5Stage::ExitStage(const FMachineContext& Context)
{
	// Ensure the transport does not have pending buffers (i.e. event data not yet processed).
	if (!Transport.IsEmpty())
	{
		Context.EmitMessage(EAnalysisMessageSeverity::Warning, TEXT("Transport buffers are not empty at end of analysis (protocol 5)!"));
	}

#if UE_TRACE_ANALYSIS_DEBUG
	Context.Bridge.GetSerial().Value = NextSerial;
	Transport.DebugEnd();
#endif
}

////////////////////////////////////////////////////////////////////////////////
FProtocol5Stage::EStatus FProtocol5Stage::OnData(
	FStreamReader& Reader,
	const FMachineContext& Context)
{
	Transport.SetReader(Reader);
	const FTidPacketTransport::ETransportResult Result = Transport.Update();
	if (Result == FTidPacketTransport::ETransportResult::Error)
	{
		Context.EmitMessage(
			EAnalysisMessageSeverity::Error,
			TEXT("An error was detected in the transport layer, most likely due to a corrupt trace file. See log for details.")
		);
		return EStatus::Error;
	}

	do
	{
		// New-events. They must be processed before anything else otherwise events
		// can not be interpreted.
		EStatus Ret = OnDataNewEvents(Context);
		if (Ret == EStatus::Error)
		{
			return Ret;
		}

		// Important events
		Ret = OnDataImportant(Context);
		if (Ret == EStatus::Error)
		{
			return Ret;
		}
		bool bNotEnoughData = (Ret == EStatus::NotEnoughData);

		// Pseudo Important events, events that should be treated
		// as important but written like regular event (e.g. during OnConnect).
		Ret = OnDataPseudoImportant(Context);
		if (Ret == EStatus::Error)
		{
			return Ret;
		}
		bNotEnoughData |= (Ret == EStatus::NotEnoughData);

		// Normal events
		Ret = OnDataNormal(Context);
		if (Ret == EStatus::Error)
		{
			return Ret;
		}
		if (Ret == EStatus::Sync)
		{
			// After processing a SYNC packet, we need to read data once more.
			return OnData(Reader, Context);
		}
		bNotEnoughData |= (Ret == EStatus::NotEnoughData);

		if (bNotEnoughData && !bSkipSerial)
		{
			return EStatus::NotEnoughData;
		}
	}
	while (bSkipSerial);

	return Reader.CanMeetDemand() ? EStatus::Continue : EStatus::EndOfStream;
}

////////////////////////////////////////////////////////////////////////////////
FProtocol5Stage::EStatus FProtocol5Stage::OnDataNewEvents(const FMachineContext& Context)
{
	EventDescs.Reset();

	FStreamReader* ThreadReader = Transport.GetKnownStream(FTidPacketTransport::EThreadStreamIndex::Events);
	if (ThreadReader->IsEmpty())
	{
		return EStatus::EndOfStream;
	}

	if (ParseImportantEvents(*ThreadReader, EventDescs, Context) < 0)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Failed to parse important events");
		return EStatus::Error;
	}

	for (const FEventDesc& EventDesc : EventDescs)
	{
		const FTypeRegistry::FTypeInfo* TypeInfo = TypeRegistry.Add(EventDesc.Data, EventVersion);
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogNewEvent(uint32(EventDesc.Uid), TypeInfo, EventDesc.EventSize + EventDesc.AuxSize);
#endif
		Context.Bridge.OnNewType(TypeInfo);
	}

	return EStatus::EndOfStream;
}

////////////////////////////////////////////////////////////////////////////////
FProtocol5Stage::EStatus FProtocol5Stage::OnDataImportant(const FMachineContext& Context)
{
	static_assert(ETransportTid::Importants == ETransportTid::Internal, "It is assumed there is only one 'important' thread stream");

	EventDescs.Reset();

	FStreamReader* ThreadReader = Transport.GetKnownStream(FTidPacketTransport::EThreadStreamIndex::Importants);
	if (ThreadReader->IsEmpty())
	{
		return EStatus::EndOfStream;
	}

	if (ParseImportantEvents(*ThreadReader, EventDescs, Context) < 0)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Failed to parse important events");
		return EStatus::Error;
	}

	bool bNotEnoughData = !ThreadReader->IsEmpty();

	if (EventDescs.Num() <= 0)
	{
		return bNotEnoughData ? EStatus::NotEnoughData : EStatus::EndOfStream;
	}

	// Dispatch looks ahead to the next desc looking for runs of aux blobs. As
	// such we should add a terminal desc for it to read. Note the "- 1" too.
	FEventDesc& EventDesc = EventDescs.Emplace_GetRef();
	EventDesc.Serial = ESerial::Terminal;

	Context.Bridge.SetActiveThread(ETransportTid::Importants);
	if (DispatchEvents(Context, EventDescs.GetData(), EventDescs.Num() - 1) < 0)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Failed to dispatch important events");
		return EStatus::Error;
	}

	return bNotEnoughData ? EStatus::NotEnoughData : EStatus::EndOfStream;
}

////////////////////////////////////////////////////////////////////////////////
FProtocol5Stage::EStatus FProtocol5Stage::OnDataPseudoImportant(const FMachineContext& Context)
{
	EventDescs.Reset();

	FStreamReader* ThreadReader = Transport.GetKnownStream(FTidPacketTransport::EThreadStreamIndex::PseudoImportants);
	if (ThreadReader->IsEmpty())
	{
		return EStatus::EndOfStream;
	}

	if (ParseEvents(*ThreadReader, EventDescs, Context) < 0)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Failed to parse on connect events");
		return EStatus::Error;
	}

	bool bNotEnoughData = !ThreadReader->IsEmpty();

	if (EventDescs.Num() <= 0)
	{
		return bNotEnoughData ? EStatus::NotEnoughData : EStatus::EndOfStream;
	}
	
	// Make sure serial events on the OnConnect thread is processed as 
	// no-sync events.
	for (auto& Event : EventDescs)
	{
		if (Event.Serial != ESerial::Ignored)
		{
			// TraceLog emits serial events on the OnConnect thread
			// with serial value '0'.
			if (Event.Serial != 0)
			{
				UE_TRACE_ANALYSIS_DEBUG_LOG("Warning: An event in PseudoImportant thread contained a non-zero serial.");
			}
			Event.Serial = ESerial::Ignored;
		}
	}

	// Dispatch looks ahead to the next desc looking for runs of aux blobs. As
	// such we should add a terminal desc for it to read. Note the "- 1" too.
	FEventDesc& EventDesc = EventDescs.Emplace_GetRef();
	EventDesc.Serial = ESerial::Terminal;

	Context.Bridge.SetActiveThread(ETransportTid::PseudoImportants);
	if (DispatchEvents(Context, EventDescs.GetData(), EventDescs.Num() - 1) < 0)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Failed to dispatch pseudo important events");
		return EStatus::Error;
	}

	return bNotEnoughData ? EStatus::NotEnoughData : EStatus::EndOfStream;
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol5Stage::ParseImportantEvents(FStreamReader& Reader, EventDescArray& OutEventDescs, const FMachineContext& Context)
{
	using namespace Protocol5;

	while (true)
	{
		uint32 Remaining = Reader.GetRemaining();
		if (Remaining < sizeof(FImportantEventHeader))
		{
			return 1;
		}

		const auto* Header = Reader.GetPointerUnchecked<FImportantEventHeader>();
		if (Remaining < uint32(Header->Size) + sizeof(FImportantEventHeader))
		{
			return 1;
		}

		uint32 Uid = Header->Uid;

		FEventDesc EventDesc;
		EventDesc.Serial = ESerial::Ignored;
		EventDesc.Uid = (uint16)Uid;
		EventDesc.Data = Header->Data;

		// Special case for new events. It would work to add a 0 type to the
		// registry but this way avoid raveling things together.
		if (Uid == EKnownUids::NewEvent)
		{
#if UE_TRACE_ANALYSIS_DEBUG
			EventDesc.EventSize = sizeof(*Header) + Header->Size;
#endif
			OutEventDescs.Add(EventDesc);
			Reader.Advance(sizeof(*Header) + Header->Size);
			continue;
		}

		const FTypeRegistry::FTypeInfo* TypeInfo = TypeRegistry.Get(Uid);
		if (TypeInfo == nullptr)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("UID %u (0x%X) was not declared yet.", Uid, Uid);
			return 1;
		}

#if UE_TRACE_ANALYSIS_DEBUG
		EventDesc.EventSize = sizeof(*Header) + TypeInfo->EventSize;
#endif

		if (TypeInfo->MaybeHasAux())
		{
			EventDesc.bHasAux = 1;
		}

		OutEventDescs.Add(EventDesc);

		if (TypeInfo->MaybeHasAux())
		{
			const uint8* Cursor = Header->Data + TypeInfo->EventSize;
			const uint8* End = Header->Data + Header->Size;
			while (Cursor <= End)
			{
				if (Cursor[0] == uint8(EKnownUids::AuxDataTerminal))
				{
					break;
				}

				check(Cursor[0] == uint8(EKnownUids::AuxData));
				const auto* AuxHeader = (FAuxHeader*)Cursor;

				FEventDesc& AuxDesc = OutEventDescs.Emplace_GetRef();
				AuxDesc.Uid = uint8(EKnownUids::AuxData);
				AuxDesc.Data = AuxHeader->Data;
				AuxDesc.Serial = ESerial::Ignored;

				Cursor = AuxHeader->Data + (AuxHeader->Pack >> FAuxHeader::SizeShift);

#if UE_TRACE_ANALYSIS_DEBUG
				AuxDesc.EventSize = sizeof(FAuxHeader);
				AuxDesc.AuxSize = (AuxHeader->Pack >> FAuxHeader::SizeShift);
#endif
			}

			if (Cursor[0] != uint8(EKnownUids::AuxDataTerminal))
			{
				UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Expecting AuxDataTerminal event");
				Context.EmitMessage(EAnalysisMessageSeverity::Warning, TEXT("Expected an aux data terminal in the stream."));
				return -1;
			}
		}

		Reader.Advance(sizeof(*Header) + Header->Size);
	}
}

////////////////////////////////////////////////////////////////////////////////
FProtocol5Stage::EStatus FProtocol5Stage::OnDataNormal(const FMachineContext& Context)
{
	// Ordinary events

	EventDescs.Reset();
	bool bNotEnoughData = false;

	TArray<FEventDescStream> EventDescHeap;
	EventDescHeap.Reserve(Transport.GetThreadCount());

	bSkipSerial = false;

	for (uint32 i = (uint32)FTidPacketTransport::EThreadStreamIndex::RealThreadsStart, n = Transport.GetThreadCount(); i < n; ++i)
	{
		uint32 NumEventDescs = EventDescs.Num();

		UE_TRACE_ANALYSIS_DEBUG_LOG_EX(3, "Thread:%u Id:%u", i, Transport.GetThreadId(i));

		// Extract all the events in the stream for this thread
		FStreamReader* ThreadReader = Transport.GetThreadStream(i);

		// Test if analysis has accumulated too much data for this thread.
		// This can happen on corrupted traces (ex. with serial sync events missing or out of order).
		constexpr uint32 MaxAccumulatedBytes = 2'000'000'000u;
		if (ThreadReader->GetRemaining() > MaxAccumulatedBytes)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Trace analysis accumulated too much data (%.2f MiB on thread %u) and will start to skip the missing serial sync events!",
				(double)ThreadReader->GetRemaining() / (1024.0 * 1024.0),
				Transport.GetThreadId(i));
			if (!bSkipSerialError)
			{
				bSkipSerialError = true;
				Context.EmitMessagef(
					EAnalysisMessageSeverity::Error,
					TEXT("Trace analysis accumulated too much data (%.2f MiB on thread %u) and will start to skip the missing serial sync events!"),
					(double)ThreadReader->GetRemaining() / (1024.0 * 1024.0),
					Transport.GetThreadId(i)
				);
			}
			bSkipSerial = true;
		}

		if (ParseEvents(*ThreadReader, EventDescs, Context) < 0)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Failed to parse events");
			return EStatus::Error;
		}

		bNotEnoughData |= !ThreadReader->IsEmpty();

		if (uint32(EventDescs.Num()) != NumEventDescs)
		{
			// Add a dummy event to delineate the end of this thread's events
			FEventDesc& EventDesc = EventDescs.Emplace_GetRef();
			EventDesc.Serial = ESerial::Terminal;

			FEventDescStream Out;
			Out.ThreadId = Transport.GetThreadId(i);
			Out.TransportIndex = i;
			Out.ContainerIndex = NumEventDescs;
			EventDescHeap.Add(Out);
		}

		UE_TRACE_ANALYSIS_DEBUG_LOG_EX(3, "Thread:%u bNotEnoughData:%d", i, bNotEnoughData);
	}

	// Now EventDescs is stable we can convert the indices into pointers
	for (FEventDescStream& Stream : EventDescHeap)
	{
		Stream.EventDescs = EventDescs.GetData() + Stream.ContainerIndex;
	}

#if UE_TRACE_ANALYSIS_DEBUG
	FAnalysisState& State = Context.Bridge.GetState();
	if (EventDescs.Num() > State.MaxEventDescs)
	{
		State.MaxEventDescs = EventDescs.Num();
	}
#endif

	const bool bSync = (SyncCount != Transport.GetSyncCount());

	int32 NumAvailableEvents = EventDescs.Num();

	// Try to dispatch the parsed events.
	{
		int32 NumDispatchedEvents = DispatchNormalEvents(Context, EventDescHeap);
		if (NumDispatchedEvents < 0)
		{
			return EStatus::Error;
		}
		NumAvailableEvents += SerialGaps.Num(); // serial gaps are detected during DispatchNormalEvents
		UE_TRACE_ANALYSIS_DEBUG_LOG_EX(2, "Dispatched %d normal events (%d --> %d)", NumDispatchedEvents, NumAvailableEvents, NumAvailableEvents - NumDispatchedEvents);
		NumAvailableEvents -= NumDispatchedEvents;
		check(NumAvailableEvents >= 0);

		// Count how many times we dispatched events, but without dispatching any "sync" event.
		if (OldNextSerial == NextSerial)
		{
			++NextSerialWaitCount;
		}
		else
		{
			OldNextSerial = NextSerial;
			NextSerialWaitCount = 0;
		}
	}

	// Test if analysis has accumulated too much data (parsed events not dispatched yet).
	// But, only enforce the limit after we have received at least one SYNC package
	// (e.g. server traces can accumulate large amounts of data before first SYNC event).
	constexpr int32 MaxAvailableEventsHighLimit = 90'000'000;
	constexpr int32 MaxAvailableEventsLowLimit = 50'000'000;
	constexpr uint32 MaxNextSerialWaitCount = 20;
	bool bSkipSerialNow = false;
	if (SyncCount > 0 &&
		!bSkipSerialError &&
		NumAvailableEvents > MaxAvailableEventsHighLimit &&
		NextSerialWaitCount > MaxNextSerialWaitCount)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Trace analysis accumulated too much data (%d parsed events) and will start to skip the missing serial sync events!", NumAvailableEvents);
		UE_TRACE_ANALYSIS_DEBUG_LOG("NextSerial=%u", NextSerial);

		if (!bSkipSerialError)
		{
			bSkipSerialError = true;
#if UE_TRACE_ANALYSIS_DEBUG
			if (UE::Trace::Analysis::GDebugLevel >= 2)
			{
				for (const auto& Stream : EventDescHeap)
				{
					const FEventDesc* EventDesc = Stream.EventDescs;
					while (EventDesc->Serial != ESerial::Terminal)
					{
						if (EventDesc->Serial == NextSerial)
						{
							UE_TRACE_ANALYSIS_DEBUG_LOG("Found next serial on thread %u (event %d)", Stream.ThreadId, (int32)(EventDesc - Stream.EventDescs));
						}
						++EventDesc;
					}
				}
			}
#endif // UE_TRACE_ANALYSIS_DEBUG
			Context.EmitMessagef(
				EAnalysisMessageSeverity::Error,
				TEXT("Trace analysis accumulated too much data (%d parsed events) and will start to skip the missing serial sync events!"), NumAvailableEvents);
		}
		bSkipSerialNow = true;
	}
	if (bSkipSerialNow ||
		(bSkipSerialError && NumAvailableEvents > MaxAvailableEventsLowLimit))
	{
		do
		{
			// Skip serials and continue to dispatch parsed events.
			bSkipSerial = true;
			NextSerialWaitCount = 0;
			int32 NumDispatchedEvents = DispatchNormalEvents(Context, EventDescHeap);
			if (NumDispatchedEvents < 0)
			{
				return EStatus::Error;
			}
			UE_TRACE_ANALYSIS_DEBUG_LOG_EX(2, "Skipped serials and dispatched %d normal events (%d --> %d)", NumDispatchedEvents, NumAvailableEvents, NumAvailableEvents - NumDispatchedEvents);
			NumAvailableEvents -= NumDispatchedEvents;
			check(NumAvailableEvents >= 0);
		}
		while (NumAvailableEvents > MaxAvailableEventsLowLimit);
	}

#if 0
	// If the reader is empty (no more incoming trace data) and we still have parsed events not dispatched,
	// then we'll try to skip the missing serial sync events.
	if (Transport.GetReader()->IsEmpty() && NumAvailableEvents > 0)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Trace reader is empty, but there are still %d parsed events not dispatched. Starting to skip the missing serial sync events!", NumAvailableEvents);
		Context.EmitMessagef(
			EAnalysisMessageSeverity::Error,
			TEXT("Trace reader is empty, but there are still %d parsed events not dispatched. Starting to skip the missing serial sync events!"), NumAvailableEvents);
		while (true)
		{
			// Skip serials and continue to dispatch parsed events.
			bSkipSerial = true;
			NextSerialWaitCount = 0;
			int32 NumDispatchedEvents = DispatchNormalEvents(Context, EventDescHeap);
			if (NumDispatchedEvents < 0)
			{
				return EStatus::Error;
			}
			UE_TRACE_ANALYSIS_DEBUG_LOG_EX(2, "Skipped serials and dispatched %d normal events (%d --> %d)", NumDispatchedEvents, NumAvailableEvents, NumAvailableEvents - NumDispatchedEvents);
			NumAvailableEvents -= NumDispatchedEvents;
			check(NumAvailableEvents >= 0);

			if (NumAvailableEvents == 0)
			{
				break;
			}
			if (NumDispatchedEvents == 0)
			{
				UE_TRACE_ANALYSIS_DEBUG_LOG("Cannot dispatch the remaining %d (incomplete) parsed events!", NumAvailableEvents);
				break;
			}
		}
		bNotEnoughData = false;
		bSkipSerial = false;
	}
#endif

	// If there are any streams left in the heap then we are unable to proceed
	// until more data is received. We'll rewind the streams until more data is
	// available. It is not an efficient way to do things, but it is simple way.
	for (FEventDescStream& Stream : EventDescHeap)
	{
		const FEventDesc& EventDesc = Stream.EventDescs[0];
		uint32 HeaderSize = 1 + EventDesc.bTwoByteUid + (ESerial::Bits / 8);

		FStreamReader* Reader = Transport.GetThreadStream(Stream.TransportIndex);
		Reader->Backtrack(EventDesc.Data - HeaderSize);
	}

	if (bSync && SyncCount == Transport.GetSyncCount())
	{
		return EStatus::Sync;
	}
	if (bNotEnoughData)
	{
		return EStatus::NotEnoughData;
	}
	return EStatus::EndOfStream;
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol5Stage::DispatchNormalEvents(const FMachineContext& Context, TArray<FEventDescStream>& EventDescHeap)
{
	UE_TRACE_ANALYSIS_DEBUG_LOG_EX(2, "Queued event descs: %d", EventDescs.Num());

	// Process leading unsynchronized events so that each stream starts with a synchronized event.
	int32 NumDispatchedUnsyncEvents = 0;
	for (FEventDescStream& Stream : EventDescHeap)
	{
		// Extract a run of consecutive unsynchronized events
		const FEventDesc* EndDesc = Stream.EventDescs;
		for (; EndDesc->Serial == ESerial::Ignored; ++EndDesc);

		// Dispatch.
		const FEventDesc* StartDesc = Stream.EventDescs;
		int32 DescNum = int32(UPTRINT(EndDesc - StartDesc));
		if (DescNum > 0)
		{
			NumDispatchedUnsyncEvents += DescNum;

			Context.Bridge.SetActiveThread(Stream.ThreadId);

			if (DispatchEvents(Context, StartDesc, DescNum) < 0)
			{
				UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Failed to dispatch events");
				return -1;
			}

			Stream.EventDescs = EndDesc;
		}
	}

	// Trim off empty streams
	int32 NumTerminators = EventDescHeap.Num();
	EventDescHeap.RemoveAllSwap([] (const FEventDescStream& Stream)
	{
		return (Stream.EventDescs->Serial == ESerial::Terminal);
	});
	NumTerminators -= EventDescHeap.Num();

	// Early out if there isn't any events available.
	if (UNLIKELY(EventDescHeap.IsEmpty()))
	{
		return NumDispatchedUnsyncEvents + NumTerminators;
	}

	// A min-heap is used to peel off groups of events by lowest serial
	EventDescHeap.Heapify(FSerialDistancePredicate{NextSerial});

#if UE_TRACE_ANALYSIS_DEBUG
	if (UE::Trace::Analysis::GDebugLevel >= 2)
	{
		const FEventDescStream& TopStream = EventDescHeap.HeapTop();
		UE_TRACE_ANALYSIS_DEBUG_LOG("NextSerial=%u LowestSerial=%d (Tid=%u)", NextSerial, TopStream.EventDescs[0].Serial, TopStream.ThreadId);
	}
#endif

	// Events must be consumed contiguously.
	if (bSkipSerial)
	{
		uint32 LowestSerial = EventDescHeap.HeapTop().EventDescs[0].Serial;
		if (LowestSerial != NextSerial)
		{
#if UE_TRACE_ANALYSIS_DEBUG
			uint32 SerialsToSkip = ((LowestSerial & ESerial::Mask) + ESerial::Range - (NextSerial & ESerial::Mask)) & ESerial::Mask;
			Context.Bridge.GetState().NumSkippedSerials += SerialsToSkip;
			UE_TRACE_ANALYSIS_DEBUG_LOG("Warning: NextSerial skips %u events (from %u to %u)", SerialsToSkip, NextSerial, LowestSerial);
#endif
			NextSerial = LowestSerial;
		}
	}
	else
	if (NextSerial == ~0u)
	{
		uint32 LowestSerial = EventDescHeap.HeapTop().EventDescs[0].Serial;
		if (Transport.GetSyncCount() || LowestSerial == 0)
		{
			NextSerial = LowestSerial;
			UE_TRACE_ANALYSIS_DEBUG_LOG("NextSerial=%u", NextSerial);
		}
	}

	DetectSerialGaps(EventDescHeap);

	int32 NumDispatchedEvents = DispatchEvents(Context, EventDescHeap);
	if (NumDispatchedEvents < 0)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Failed to dispatch events");
		return -1;
	}

	return NumDispatchedUnsyncEvents + NumTerminators + NumDispatchedEvents;
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol5Stage::DispatchEvents(const FMachineContext& Context, TArray<FEventDescStream>& EventDescHeap)
{
#if UE_TRACE_ANALYSIS_DEBUG
	FAnalysisState& State = Context.Bridge.GetState();
#endif

	auto UpdateHeap = [&] (const FEventDescStream& Stream, const FEventDesc* EventDesc)
	{
		if (EventDesc->Serial != ESerial::Terminal)
		{
			// Stream reference is for an element of EventDescHeap array, so we copy it before modifying the array.
			FEventDescStream Next = Stream;
			Next.EventDescs = EventDesc;
			// We can now discard the top stream from the heap and push the updated stream to the heap.
			EventDescHeap.HeapPopDiscard(FSerialDistancePredicate{ NextSerial }, EAllowShrinking::No);
			EventDescHeap.HeapPush(Next, FSerialDistancePredicate{ NextSerial });
		}
		else
		{
			EventDescHeap.HeapPopDiscard(FSerialDistancePredicate{ NextSerial }, EAllowShrinking::No);
		}
	};

	int32 NumDispatchedEvents = 0;

	do
	{
		const FEventDescStream& Stream = EventDescHeap.HeapTop();
		const FEventDesc* StartDesc = Stream.EventDescs;
		const FEventDesc* EndDesc = StartDesc;

		// DetectSerialGaps() will add a special stream that communicates gaps
		// in serial numbers, gaps that will never be resolved. Thread IDs
		// are uint16 everywhere else so they will never collide with GapThreadId.
		if (Stream.ThreadId == FEventDescStream::GapThreadId)
		{
			NextSerial = EndDesc->Serial + EndDesc->GapLength;
			NextSerial &= ESerial::Mask;
#if UE_TRACE_ANALYSIS_DEBUG
			if (NextSerial != EndDesc->Serial + EndDesc->GapLength)
			{
				State.SerialWrappedCount++;
			}
			State.NumSkippedSerialGaps++;
			UE_TRACE_ANALYSIS_DEBUG_LOG("Skip serial gap (%u +%u) --> NextSerial=%u", EndDesc->Serial, EndDesc->GapLength, NextSerial);
#endif
			UpdateHeap(Stream, EndDesc + 1);
			NumDispatchedEvents += ((EndDesc + 1)->Serial == ESerial::Terminal) ? 2 : 1; // GapThreadId event + Terminal event
			continue;
		}

		// Extract a run of consecutive events (plus runs of unsynchronized ones).
		if (EndDesc->Serial == NextSerial)
		{
#if UE_TRACE_ANALYSIS_DEBUG
			const uint32 CurrentSerial = NextSerial;
#endif

			do
			{
				NextSerial = (NextSerial + 1) & ESerial::Mask;

				do
				{
					++EndDesc;
				}
				while (EndDesc->Serial == ESerial::Ignored);
			}
			while (EndDesc->Serial == NextSerial);

#if UE_TRACE_ANALYSIS_DEBUG
			if (NextSerial < CurrentSerial)
			{
				++State.SerialWrappedCount;
			}
#endif
		}
		else
		{
#if UE_TRACE_ANALYSIS_DEBUG
			if (uint32(EndDesc->Serial) < NextSerial &&
				NextSerial != ~0u &&
				NextSerial - uint32(EndDesc->Serial) < ESerial::Range/2)
			{
				UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Lowest serial %d (Tid=%u Uid=%u) is too low (NextSerial=%u; %d event descs) !!!", EndDesc->Serial, Stream.ThreadId, uint32(EndDesc->Uid), NextSerial, EventDescs.Num());
			}
			else
			{
				UE_TRACE_ANALYSIS_DEBUG_LOG_EX(2, "Lowest serial %d (Tid=%u Uid=%u) is not low enough (NextSerial=%u; %d event descs)", EndDesc->Serial, Stream.ThreadId, uint32(EndDesc->Uid), NextSerial, EventDescs.Num());
			}

			if (UE::Trace::Analysis::GDebugLevel >= 2)
			{
				UE_TRACE_ANALYSIS_DEBUG_LOG("Available streams (SerialWrappedCount=%u):", State.SerialWrappedCount);
				for (const FEventDescStream& EventDescStream : EventDescHeap)
				{
					uint32 BufferSize = 0;
					uint32 DataSize = 0; // parsed data size + remaining data size
					uint32 RemainingSize = 0;
					for (uint32 i = 0, n = Transport.GetThreadCount(); i < n; ++i)
					{
						FStreamBuffer* ThreadReader = (FStreamBuffer*)Transport.GetThreadStream(i);
						uint32 ThreadId = Transport.GetThreadId(i);
						if (ThreadId == EventDescStream.ThreadId)
						{
							BufferSize = ThreadReader->GetBufferSize();
							const FEventDesc& EventDesc = EventDescStream.EventDescs[0];
							uint32 HeaderSize = 1 + EventDesc.bTwoByteUid + (ESerial::Bits / 8);
							DataSize = ThreadReader->GetBacktrackSize(EventDesc.Data - HeaderSize);
							RemainingSize = ThreadReader->GetRemaining();
							break;
						}
					}
					UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder();
					UE_TRACE_ANALYSIS_DEBUG_Appendf("  Tid=%u : Serial=%u BufferSize=%u DataSize=%u", EventDescStream.ThreadId, EventDescStream.EventDescs->Serial, BufferSize, DataSize);
					if (RemainingSize != 0)
					{
						UE_TRACE_ANALYSIS_DEBUG_Appendf(" (%u + %u)", DataSize - RemainingSize, RemainingSize);
					}
					if (EventDescStream.EventDescs->Serial == NextSerial)
					{
						UE_TRACE_ANALYSIS_DEBUG_Append(" (next)");
					}
					UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder();
				}
			}
#endif // UE_TRACE_ANALYSIS_DEBUG

#if 0
			int32 MinSerial = EndDesc->Serial;
			EventDescHeap.Heapify(FSerialDistancePredicate{NextSerial});
			const FEventDescStream& MinStream = EventDescHeap.HeapTop();
			const FEventDesc* MinDesc = MinStream.EventDescs;
			if (MinDesc->Serial < MinSerial)
			{
				UE_TRACE_ANALYSIS_DEBUG_LOG("Try one more time with lowest serial %d (Tid=%u Uid=%u)", MinDesc->Serial, MinStream.ThreadId, uint32(MinDesc->Uid));
				continue;
			}
#endif

			// The lowest known serial number is not low enough so we are unable to proceed any further.
			break;
		}

		// Dispatch.
		Context.Bridge.SetActiveThread(Stream.ThreadId);
		int32 DescNum = int32(UPTRINT(EndDesc - StartDesc));
		check(DescNum > 0);
		NumDispatchedEvents += DescNum;
		if (DispatchEvents(Context, StartDesc, DescNum) < 0)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Failed to dispatch events");
			return -1;
		}

		UpdateHeap(Stream, EndDesc);
		NumDispatchedEvents += (EndDesc->Serial == ESerial::Terminal) ? 1 : 0; // Terminal event
	}
	while (!EventDescHeap.IsEmpty());

	return NumDispatchedEvents;
}

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ANALYSIS_DEBUG
void FProtocol5Stage::PrintParsedEvent(int EventIndex, const FEventDesc& EventDesc, int32 Size)
{
	UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder();
	UE_TRACE_ANALYSIS_DEBUG_Appendf("Event=%-6d Uid=%-4u ", EventIndex, uint32(EventDesc.Uid));
	if (EventDesc.Serial >= ESerial::Range)
	{
		UE_TRACE_ANALYSIS_DEBUG_Appendf("Serial=0x%07X ", EventDesc.Serial);
	}
	else
	{
		UE_TRACE_ANALYSIS_DEBUG_Appendf("Serial=%-9d ", EventDesc.Serial);
	}
	UE_TRACE_ANALYSIS_DEBUG_Appendf("Size=%d", Size);
	if (EventDesc.bHasAux)
	{
		UE_TRACE_ANALYSIS_DEBUG_Append(" aux");
	}
	if (EventDesc.Uid == EKnownUids::AuxData)
	{
		UE_TRACE_ANALYSIS_DEBUG_Append(" data");
	}
	else if (EventDesc.Uid == EKnownUids::AuxDataTerminal)
	{
		UE_TRACE_ANALYSIS_DEBUG_Append(" end");
	}
	UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder();
}
#endif // UE_TRACE_ANALYSIS_DEBUG

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol5Stage::ParseEvents(FStreamReader& Reader, EventDescArray& OutEventDescs, const FMachineContext& Context)
{
	while (!Reader.IsEmpty())
	{
		FEventDesc EventDesc;

		int32 Size = ParseEvent(Reader, EventDesc, Context);
		if (Size <= 0)
		{
			return Size;
		}

#if UE_TRACE_ANALYSIS_DEBUG
		if (UE::Trace::Analysis::GDebugLevel >= 3)
		{
			PrintParsedEvent(OutEventDescs.Num(), EventDesc, Size);
		}
#endif // UE_TRACE_ANALYSIS_DEBUG

		OutEventDescs.Add(EventDesc);

		if (EventDesc.bHasAux)
		{
			uint32 RewindDescsNum = OutEventDescs.Num() - 1;
			auto RewindMark = Reader.SaveMark();

			Reader.Advance(Size);

			int Ok = ParseEventsWithAux(Reader, OutEventDescs, Context);
			if (Ok < 0)
			{
				return Ok;
			}

			if (Ok == 0)
			{
				UE_TRACE_ANALYSIS_DEBUG_LOG_EX(2, "Warning: Incomplete aux stack (Uid=%u)! Rewind %d parsed events.", EventDesc.Uid, OutEventDescs.Num() - RewindDescsNum);
				OutEventDescs.SetNum(RewindDescsNum, EAllowShrinking::No);
				Reader.RestoreMark(RewindMark);
				break;
			}

			continue;
		}

		Reader.Advance(Size);
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol5Stage::ParseEventsWithAux(FStreamReader& Reader, EventDescArray& OutEventDescs, const FMachineContext& Context)
{
	// We are now "in" the scope of an event with zero or more aux-data blocks.
	// We will consume events until we leave this scope (a aux-data-terminal).
	// A running key is assigned to each event with a gap left following events
	// that may have aux-data blocks. Aux-data blocks are assigned a key that
	// fits in these gaps. Once sorted by this key, events maintain their order
	// while aux-data blocks are moved to directly follow their owners.

	TArray<uint16, TInlineAllocator<8>> AuxKeyStack = { 0 };
	uint32 AuxKey = 2;

	uint32 FirstDescIndex = OutEventDescs.Num();
	bool bUnsorted = false;

	while (!Reader.IsEmpty())
	{
		FEventDesc EventDesc;
		EventDesc.Serial = ESerial::Ignored;

		int32 Size = ParseEvent(Reader, EventDesc, Context);
		if (Size <= 0)
		{
			return Size;
		}

#if UE_TRACE_ANALYSIS_DEBUG
		if (UE::Trace::Analysis::GDebugLevel >= 3)
		{
			PrintParsedEvent(OutEventDescs.Num(), EventDesc, Size);
		}
#endif // UE_TRACE_ANALYSIS_DEBUG

		Reader.Advance(Size);

		if (EventDesc.Uid == EKnownUids::AuxDataTerminal)
		{
			// Leave the scope of an aux-owning event.
			if (AuxKeyStack.Pop() == 0)
			{
				break;
			}
			continue;
		}
		else if (EventDesc.Uid == EKnownUids::AuxData)
		{
			// Move an aux-data block to follow its owning event
			EventDesc.AuxKey = AuxKeyStack.Last() + 1;
		}
		else
		{
			EventDesc.AuxKey = uint16(AuxKey);

			// Maybe it is time to create a new aux-data owner scope
			if (EventDesc.bHasAux)
			{
				AuxKeyStack.Add(uint16(AuxKey));
			}

			// This event may be in the middle of an earlier event's aux data blocks.
			bUnsorted = true;
		}

		OutEventDescs.Add(EventDesc);

		++AuxKey;

		constexpr uint32 MaxAuxKey = 0x7fff;
		if (AuxKeyStack.Num() == 1 && AuxKey > MaxAuxKey)
		{
			// If an "aux terminal" for the initial event was not detected after
			// many intermediate events, we can assume it is lost.
			check(FirstDescIndex > 0);
			uint32 NumParsedEvents = uint32(OutEventDescs.Num()) - FirstDescIndex;
			UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Ignoring lost aux terminal for event with uid %u (desc %d), after parsing %u events",
				OutEventDescs[FirstDescIndex - 1].Uid,
				FirstDescIndex - 1,
				NumParsedEvents);
			Context.EmitMessagef(
				EAnalysisMessageSeverity::Error,
				TEXT("Ignoring lost aux terminal for event with uid %u, after parsing %u events."),
				OutEventDescs[FirstDescIndex - 1].Uid,
				NumParsedEvents);
			AuxKeyStack.Pop();
			break;
		}
	}

	if (AuxKeyStack.Num() > 0)
	{
		// There was not enough data available to complete the outer most scope
		return 0;
	}

	checkf((AuxKey & 0xffff0000) == 0, TEXT("AuxKey overflow (0x%X)"), AuxKey);

	// Sort to get all aux-blocks contiguous with their owning event
	if (bUnsorted)
	{
		uint32 NumDescs = OutEventDescs.Num() - FirstDescIndex;
		UE_TRACE_ANALYSIS_DEBUG_LOG_EX(3, "Sorting %u event descs", NumDescs);
		TArrayView<FEventDesc> DescsView(OutEventDescs.GetData() + FirstDescIndex, NumDescs);
		Algo::StableSort(
			DescsView,
			[] (const FEventDesc& Lhs, const FEventDesc& Rhs)
			{
				return Lhs.AuxKey < Rhs.AuxKey;
			}
		);
	}

	return 1;
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol5Stage::ParseEvent(FStreamReader& Reader, FEventDesc& EventDesc, const FMachineContext& Context)
{
	using namespace Protocol5;

	// No need to aggressively bounds check here. Events are never fragmented
	// due to the way that data is transported (aux payloads can be though).
	const uint8* Cursor = Reader.GetPointerUnchecked<uint8>();

	// Parse the event's ID
	uint32 Uid = *Cursor;
	if (Uid & EKnownUids::Flag_TwoByteUid)
	{
		EventDesc.bTwoByteUid = 1;
		Uid = *(uint16*)Cursor;
		++Cursor;
	}
	Uid >>= EKnownUids::_UidShift;
	++Cursor;

	// Calculate the size of the event
	uint32 Serial = uint32(ESerial::Ignored);
	uint32 EventSize = 0;
	if (Uid < EKnownUids::User)
	{
		/* Well-known event */

		if (Uid == Protocol5::EKnownEventUids::AuxData)
		{
			--Cursor; // FAuxHeader includes the one-byte Uid
			const auto* AuxHeader = (FAuxHeader*)Cursor;

			uint32 Remaining = Reader.GetRemaining();
			uint32 Size = AuxHeader->Pack >> FAuxHeader::SizeShift;
			if (Remaining < Size + sizeof(FAuxHeader))
			{
				return 0;
			}

			EventSize = Size;
			Cursor += sizeof(FAuxHeader);
		}
		else
		{
			SetSizeIfKnownEvent(Uid, EventSize);
		}

		EventDesc.bHasAux = 0;
	}
	else
	{
		/* Ordinary events */

		const FTypeRegistry::FTypeInfo* TypeInfo = TypeRegistry.Get(Uid);
		if (TypeInfo == nullptr)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Warning: UID %u (0x%X) was not declared yet!", Uid, Uid);
			return 0;
		}

		EventSize = TypeInfo->EventSize;
		EventDesc.bHasAux = TypeInfo->MaybeHasAux();

		if (TypeInfo->IsSync())
		{
			memcpy(&Serial, Cursor, sizeof(int32));
			Serial &= ESerial::Mask;
			Cursor += 3;
		}
	}

	EventDesc.Serial = Serial;
	EventDesc.Uid = (uint16)Uid;
	EventDesc.Data = Cursor;

	uint32 HeaderSize = uint32(UPTRINT(Cursor - Reader.GetPointer<uint8>()));
	uint32 TotalEventSize = HeaderSize + EventSize;
#if UE_TRACE_ANALYSIS_DEBUG
	EventDesc.EventSize = TotalEventSize;
#endif
	return TotalEventSize;
}

////////////////////////////////////////////////////////////////////////////////
template <typename Callback>
void FProtocol5Stage::ForEachSerialGap(const TArray<FEventDescStream>& EventDescHeap, Callback&& InCallback)
{
	TArray<FEventDescStream> HeapCopy(EventDescHeap);
	int Serial = HeapCopy.HeapTop().EventDescs[0].Serial;

	// There might be a gap at the beginning of the heap if some events have
	// already been consumed.
	if (NextSerial != Serial)
	{
		if (!InCallback(NextSerial, Serial))
		{
			return;
		}
	}

	// A min-heap is used to peel off each stream (thread) with the lowest serial
	// numbered event.
	do
	{
		const FEventDescStream& Stream = HeapCopy.HeapTop();
		const FEventDesc* EventDesc = Stream.EventDescs;

		// If the next lowest serial number doesn't match where we got up to in
		// the previous stream we have found a gap. Celebration ensues.
		if (Serial != EventDesc->Serial)
		{
			if (!InCallback(Serial, EventDesc->Serial))
			{
				return;
			}
		}

		// Consume consecutive events (including unsynchronized ones).
		Serial = EventDesc->Serial;
		do
		{
			do
			{
				++EventDesc;
			}
			while (EventDesc->Serial == ESerial::Ignored);

			Serial = (Serial + 1) & ESerial::Mask;
		}
		while (EventDesc->Serial == Serial);

		// Update the heap
		if (EventDesc->Serial != ESerial::Terminal)
		{
			auto& Out = HeapCopy.Add_GetRef({Stream.ThreadId, Stream.TransportIndex});
			Out.EventDescs = EventDesc;
		}
		HeapCopy.HeapPopDiscard(FSerialDistancePredicate{NextSerial}, EAllowShrinking::No);
	}
	while (!HeapCopy.IsEmpty());
}

////////////////////////////////////////////////////////////////////////////////
void FProtocol5Stage::DetectSerialGaps(TArray<FEventDescStream>& EventDescHeap)
{
	// Events that should be synchronized across threads are assigned serial
	// numbers so they can be analyzed in the correct order. Gaps in the
	// serials can occur under two scenarios; 1) when packets are dropped from
	// the trace tail to make space for new trace events, and 2) when Trace's
	// worker thread ticks, samples all the trace buffers and sends their data.
	// In late-connect scenarios these gaps need to be skipped over in order to
	// successfully re-serialize events in the data stream. To further complicate
	// matters, most of the gaps from (2) will get filled by the following update,
	// leading to initial false positive gaps. By embedding sync points in the
	// stream we can reliably differentiate genuine gaps from temporary ones.
	//
	// Note that this could be done without sync points but it is an altogether
	// more complex solution. So unsightly embedded syncs it is...

	if (SyncCount == Transport.GetSyncCount())
	{
		return;
	}

	SyncCount = Transport.GetSyncCount();
	UE_TRACE_ANALYSIS_DEBUG_LOG("SyncCount: %d (%d previous serial gaps)", SyncCount, SerialGaps.Num());

	// We expect 3 syncpoints before we consider the stream complete. Once the second
	// sync is parsed we can collect the gaps in serials
	if (SyncCount == 2)
	{
		// On the first update we will just collect gaps.
		auto GatherGap = [this] (int32 Lhs, int32 Rhs)
		{
			FEventDesc& Gap = SerialGaps.Emplace_GetRef();
			Gap.Serial = Lhs;
			Gap.GapLength = (Rhs - Lhs) & ESerial::Mask;
			return true;
		};
		ForEachSerialGap(EventDescHeap, GatherGap);
	}
	else if (SyncCount == 3)
	{
		// On the third update we detect where gaps from the previous update
		// start getting filled in. Any gaps preceding that point are genuine.
		uint32 GapCount = 0;
		auto RecordGap = [this, &GapCount] (int32 Lhs, int32 Rhs)
		{
			if (SerialGaps.IsEmpty() || GapCount >= (uint32)SerialGaps.Num())
			{
				return false;
			}

			const FEventDesc& SerialGap = SerialGaps[GapCount];

			if (SerialGap.Serial == Lhs)
			{
				/* This is the expected case */
				++GapCount;
				return true;
			}

			if (SerialGap.Serial > Lhs)
			{
				/* We've started receiving new gaps that are exist because not all
				* data has been received yet. They're false positives. No need to process
				* any further */
				return false;
			}

			// If we're here something's probably gone wrong
			UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Serial gaps detection failed (SerialGap.Serial=%d, Lhs=%d)!", SerialGap.Serial, Lhs);
			return false;
		};

		ForEachSerialGap(EventDescHeap, RecordGap);

		UE_TRACE_ANALYSIS_DEBUG_LOG("Serial gaps: %d", GapCount);

		if (GapCount == 0)
		{
			SerialGaps.Empty();
			return;
		}

#if UE_TRACE_ANALYSIS_DEBUG
		if (UE::Trace::Analysis::GDebugLevel >= 2)
		{
			for (uint32 GapIndex = 0; GapIndex < GapCount; ++GapIndex)
			{
				const FEventDesc& SerialGap = SerialGaps[GapIndex];
				UE_TRACE_ANALYSIS_DEBUG_LOG("  gap %u +%u", SerialGap.Serial, SerialGap.GapLength);
			}
		}
#endif

		// Turn the genuine gaps into a stream that DispatchEvents() can handle
		// and use to skip over them.

		if (GapCount == uint32(SerialGaps.Num()))
		{
			SerialGaps.Emplace();
		}
		FEventDesc& Terminator = SerialGaps[GapCount];
		Terminator.Serial = ESerial::Terminal;

		FEventDescStream Out = EventDescHeap[0];
		Out.ThreadId = FEventDescStream::GapThreadId;
		Out.EventDescs = SerialGaps.GetData();
		EventDescHeap.HeapPush(Out, FSerialDistancePredicate{NextSerial});
	}
}

////////////////////////////////////////////////////////////////////////////////
void FProtocol5Stage::SetSizeIfKnownEvent(uint32 Uid, uint32& InOutEventSize)
{
	switch (Uid)
	{
	case EKnownUids::EnterScope_T:
	case EKnownUids::LeaveScope_T:
		InOutEventSize = 7;
		break;
	};
}

////////////////////////////////////////////////////////////////////////////////
bool FProtocol5Stage::DispatchKnownEvent(const FMachineContext& Context, uint32 Uid, const FEventDesc* Cursor)
{
	// Maybe this is a "well-known" event that is handled a little different?
	switch (Uid)
	{
	case EKnownUids::EnterScope:
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogEnterScopeEvent(Uid, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.EnterScope();
		return true;

	case EKnownUids::LeaveScope:
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogLeaveScopeEvent(Uid, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.LeaveScope();
		return true;

	case EKnownUids::EnterScope_T:
	{
		uint64 RelativeTimestamp = *(uint64*)(Cursor->Data - 1) >> 8;
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogEnterScopeEvent(Uid, RelativeTimestamp, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.EnterScope(RelativeTimestamp);
		return true;
	}

	case EKnownUids::LeaveScope_T:
	{
		uint64 RelativeTimestamp = *(uint64*)(Cursor->Data - 1) >> 8;
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogLeaveScopeEvent(Uid, RelativeTimestamp, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.LeaveScope(RelativeTimestamp);
		return true;
	}

	case EKnownUids::AuxData:
	case EKnownUids::AuxDataTerminal:
		return true;

	default:
		return false;
	}
}

////////////////////////////////////////////////////////////////////////////////
int32 FProtocol5Stage::DispatchEvents(const FMachineContext& Context, const FEventDesc* EventDesc, uint32 Count)
{
	using namespace Protocol5;

	FAuxDataCollector AuxCollector;

	UE_TRACE_ANALYSIS_DEBUG_LOG_EX(3, "Dispatch run of %u consecutive events (Tid=%u)", Count, Context.Bridge.GetActiveThreadId());

	for (const FEventDesc* Cursor = EventDesc, *End = EventDesc + Count; Cursor < End; ++Cursor)
	{
		uint32 Uid = uint32(Cursor->Uid);

		UE_TRACE_ANALYSIS_DEBUG_LOG_EX(4, "Dispatch event (Uid=%u)...", Uid);

		if (DispatchKnownEvent(Context, Uid, Cursor))
		{
			continue;
		}

		if (!TypeRegistry.IsUidValid(Uid))
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Warning: Unexpected event with UID %u (0x%X)", Uid, Uid);
			Context.EmitMessagef(EAnalysisMessageSeverity::Warning, TEXT("An unknown event UID (%u) was encountered."), Uid);

#if UE_TRACE_ANALYSIS_DEBUG
			if (UE::Trace::Analysis::GDebugLevel >= 3)
			{
				UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder();
				const uint8* StartCursor = Cursor->Data;
				for (uint32 i = 0; i < 32 && i < Cursor->EventSize; ++i)
				{
					UE_TRACE_ANALYSIS_DEBUG_Appendf("%02X ", StartCursor[i]);
				}
				UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder();
				UE_TRACE_ANALYSIS_DEBUG_ResetStringBuilder();
				UE_TRACE_ANALYSIS_DEBUG_Append("[[[");
				for (uint32 i = 0; i < 128 && i < Cursor->EventSize; ++i)
				{
					UE_TRACE_ANALYSIS_DEBUG_AppendChar((char)StartCursor[i]);
				}
				UE_TRACE_ANALYSIS_DEBUG_Append("]]]");
				UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder();
			}
#endif // UE_TRACE_ANALYSIS_DEBUG

			return -1;
		}

		// It is a normal event.
		const FTypeRegistry::FTypeInfo* TypeInfo = TypeRegistry.Get(Uid);
		FEventDataInfo EventDataInfo = {
			Cursor->Data,
			*TypeInfo,
			&AuxCollector,
			TypeInfo->EventSize,
		};

#if UE_TRACE_ANALYSIS_DEBUG
		uint32 FixedSize = Cursor->EventSize;
		uint32 AuxSize = Cursor->AuxSize;
		const FEventDesc* EventCursor = Cursor;
#endif

		// Gather its auxiliary data blocks into a collector.
		if (Cursor->bHasAux)
		{
			while (true)
			{
				++Cursor;

				if (Cursor->Uid != EKnownUids::AuxData)
				{
					--Cursor; // Read off too much. Put it back.
					break;
				}

				const auto* AuxHeader = ((FAuxHeader*)(Cursor->Data)) - 1;

				FAuxData AuxData = {};
				AuxData.Data = AuxHeader->Data;
				AuxData.DataSize = (AuxHeader->Pack >> FAuxHeader::SizeShift);
				AuxData.FieldIndex = AuxHeader->FieldIndex_Size & FAuxHeader::FieldMask;
				// AuxData.FieldSizeAndType = ... - this is assigned on demand in GetData()
				AuxCollector.Add(AuxData);

#if UE_TRACE_ANALYSIS_DEBUG
				AuxSize += Cursor->EventSize + Cursor->AuxSize;
#endif
			}
#if UE_TRACE_ANALYSIS_DEBUG
			AuxSize += 1; // for AuxDataTerminal
#endif
		}

#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogEvent(TypeInfo, FixedSize, AuxSize, EventCursor->Serial);
#endif

		Context.Bridge.OnEvent(EventDataInfo);

		AuxCollector.Reset();
	}

	return 0;
}



// {{{1 protocol-6 -------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FProtocol6Stage
	: public FProtocol5Stage
{
public:
						FProtocol6Stage(FTransport* InTransport);

	virtual void		ExitStage(const FMachineContext& Context) override;
};

////////////////////////////////////////////////////////////////////////////////
FProtocol6Stage::FProtocol6Stage(FTransport* InTransport)
	: FProtocol5Stage(InTransport)
{
	EventVersion = 6;
}

////////////////////////////////////////////////////////////////////////////////
void FProtocol6Stage::ExitStage(const FMachineContext& Context)
{
	// Ensure the transport does not have pending buffers (i.e. event data not yet processed).
	if (!Transport.IsEmpty())
	{
		Context.EmitMessage(EAnalysisMessageSeverity::Warning, TEXT("Transport buffers are not empty at end of analysis (protocol 6)!"));
	}

#if UE_TRACE_ANALYSIS_DEBUG
	Context.Bridge.GetSerial().Value = NextSerial;
	Transport.DebugEnd();
#endif
}



// {{{1 protocol-7 -------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FProtocol7Stage
	: public FProtocol6Stage
{
public:
						FProtocol7Stage(FTransport* InTransport);

	virtual void		ExitStage(const FMachineContext& Context) override;
	virtual void		SetSizeIfKnownEvent(uint32 Uid, uint32& InOutEventSize) override;
	virtual bool		DispatchKnownEvent(const FMachineContext& Context, uint32 Uid, const FEventDesc* Cursor) override;

protected:
	using EKnownUids	= Protocol7::EKnownEventUids;
};

////////////////////////////////////////////////////////////////////////////////
FProtocol7Stage::FProtocol7Stage(FTransport* InTransport)
	: FProtocol6Stage(InTransport)
{
	EventVersion = 7;
}

////////////////////////////////////////////////////////////////////////////////
void FProtocol7Stage::ExitStage(const FMachineContext& Context)
{
	// Ensure the transport does not have pending buffers (i.e. event data not yet processed).
	if (!Transport.IsEmpty())
	{
		uint64 TotalRemainingDataSize = 0;
		for (int32 ThreadIndex = 0, ThreadCount = Transport.GetThreadCount(); ThreadIndex < ThreadCount; ++ThreadIndex)
		{
			TotalRemainingDataSize += Transport.GetThreadStream(ThreadIndex)->GetRemaining();
		}
		Context.EmitMessagef(
			EAnalysisMessageSeverity::Warning,
			TEXT("Transport buffers are not empty at end of analysis (%llu bytes; protocol 7)!"), TotalRemainingDataSize);
	}

#if UE_TRACE_ANALYSIS_DEBUG
	Context.Bridge.GetSerial().Value = NextSerial;
	Transport.DebugEnd();
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FProtocol7Stage::SetSizeIfKnownEvent(uint32 Uid, uint32& InOutEventSize)
{
	switch (Uid)
	{
	case EKnownUids::EnterScope_TA:
	case EKnownUids::LeaveScope_TA:
		InOutEventSize = 8;
		break;

	case EKnownUids::EnterScope_TB:
	case EKnownUids::LeaveScope_TB:
		InOutEventSize = 7;
		break;
	};
}

////////////////////////////////////////////////////////////////////////////////
bool FProtocol7Stage::DispatchKnownEvent(const FMachineContext& Context, uint32 Uid, const FEventDesc* Cursor)
{
	// Maybe this is a "well-known" event that is handled a little different?
	switch (Uid)
	{
	case EKnownUids::EnterScope:
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogEnterScopeEvent(Uid, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.EnterScope();
		return true;

	case EKnownUids::LeaveScope:
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogLeaveScopeEvent(Uid, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.LeaveScope();
		return true;

	case EKnownUids::EnterScope_TA:
	{
		uint64 AbsoluteTimestamp = *(uint64*)Cursor->Data;
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogEnterScopeAEvent(Uid, AbsoluteTimestamp, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.EnterScopeA(AbsoluteTimestamp);
		return true;
	}

	case EKnownUids::LeaveScope_TA:
	{
		uint64 AbsoluteTimestamp = *(uint64*)Cursor->Data;
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogLeaveScopeAEvent(Uid, AbsoluteTimestamp, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.LeaveScopeA(AbsoluteTimestamp);
		return true;
	}

	case EKnownUids::EnterScope_TB:
	{
		uint64 BaseRelativeTimestamp = *(uint64*)(Cursor->Data - 1) >> 8;
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogEnterScopeBEvent(Uid, BaseRelativeTimestamp, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.EnterScopeB(BaseRelativeTimestamp);
		return true;
	}

	case EKnownUids::LeaveScope_TB:
	{
		uint64 BaseRelativeTimestamp = *(uint64*)(Cursor->Data - 1) >> 8;
#if UE_TRACE_ANALYSIS_DEBUG
		Context.Bridge.DebugLogLeaveScopeBEvent(Uid, BaseRelativeTimestamp, Cursor->EventSize + Cursor->AuxSize);
#endif
		Context.Bridge.LeaveScopeB(BaseRelativeTimestamp);
		return true;
	}

	case EKnownUids::AuxData:
	case EKnownUids::AuxDataTerminal:
		return true;

	default:
		return false;
	}
}

// {{{1 est.-transport ---------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FEstablishTransportStage
	: public FAnalysisMachine::FStage
{
public:
	virtual EStatus OnData(FStreamReader& Reader, const FMachineContext& Context) override;
};

////////////////////////////////////////////////////////////////////////////////
FEstablishTransportStage::EStatus FEstablishTransportStage::OnData(
	FStreamReader& Reader,
	const FMachineContext& Context)
{
	using namespace UE::Trace;

	const struct {
		uint8 TransportVersion;
		uint8 ProtocolVersion;
	}* Header = decltype(Header)(Reader.GetPointer(sizeof(*Header)));
	if (Header == nullptr)
	{
		return EStatus::NotEnoughData;
	}

	uint32 TransportVersion = Header->TransportVersion;
	UE_TRACE_ANALYSIS_DEBUG_LOG("TransportVersion: %u", TransportVersion);

	uint32 ProtocolVersion = Header->ProtocolVersion;
	UE_TRACE_ANALYSIS_DEBUG_LOG("ProtocolVersion: %u", ProtocolVersion);

	FTransport* Transport = nullptr;
	switch (TransportVersion)
	{
	case ETransport::Raw:			Transport = new FTransport(); break;
	case ETransport::Packet:		Transport = new FPacketTransport(); break;
	case ETransport::TidPacket:		Transport = new FTidPacketTransport(); break;
	case ETransport::TidPacketSync:	Transport = new FTidPacketTransportSync(); break;
	default:
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Invalid transport version %u", TransportVersion);
			Context.EmitMessagef(
				EAnalysisMessageSeverity::Error,
				TEXT("Unknown transport version: %u. You may need to recompile this application"),
				TransportVersion
			);
			return EStatus::Error;
		}
	}

#if UE_TRACE_ANALYSIS_DEBUG
	Transport->DebugBegin();
#endif

	switch (ProtocolVersion)
	{
	case Protocol0::EProtocol::Id:
		Context.Machine.QueueStage<FProtocol0Stage>(Transport);
		Context.Machine.Transition();
		break;

	case Protocol1::EProtocol::Id:
	case Protocol2::EProtocol::Id:
	case Protocol3::EProtocol::Id:
		Context.Machine.QueueStage<FProtocol2Stage>(ProtocolVersion, Transport);
		Context.Machine.Transition();
		break;

	case Protocol4::EProtocol::Id:
		Context.Machine.QueueStage<FProtocol4Stage>(ProtocolVersion, Transport);
		Context.Machine.Transition();
		break;

	case Protocol5::EProtocol::Id:
		Context.Machine.QueueStage<FProtocol5Stage>(Transport);
		Context.Machine.Transition();
		break;

	case Protocol6::EProtocol::Id:
		Context.Machine.QueueStage<FProtocol6Stage>(Transport);
		Context.Machine.Transition();
		break;

	case Protocol7::EProtocol::Id:
		Context.Machine.QueueStage<FProtocol7Stage>(Transport);
		Context.Machine.Transition();
		break;

	default:
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Invalid protocol version %u", ProtocolVersion);
			Context.EmitMessagef(
				EAnalysisMessageSeverity::Error,
				TEXT("Unknown protocol version: %u. You may need to recompile this application"),
				ProtocolVersion
			);
			return EStatus::Error;
		}
	}

	UE_TRACE_ANALYSIS_DEBUG_LOG("");

	Reader.Advance(sizeof(*Header));

	Context.Bridge.OnVersion(TransportVersion, ProtocolVersion);

	return EStatus::Continue;
}



// {{{1 metadata ---------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FMetadataStage
	: public FAnalysisMachine::FStage
{
public:
	virtual EStatus OnData(FStreamReader& Reader, const FMachineContext& Context) override;
};

////////////////////////////////////////////////////////////////////////////////
FMetadataStage::EStatus FMetadataStage::OnData(
	FStreamReader& Reader,
	const FMachineContext& Context)
{
	const auto* MetadataSize = Reader.GetPointer<uint16>();
	if (MetadataSize == nullptr)
	{
		return EStatus::NotEnoughData;
	}

	const uint8* Metadata = Reader.GetPointer(sizeof(*MetadataSize) + *MetadataSize);
	if (Metadata == nullptr)
	{
		return EStatus::NotEnoughData;
	}

#if UE_TRACE_ANALYSIS_DEBUG
	UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder();
	UE_TRACE_ANALYSIS_DEBUG_Appendf("Metadata: %u + %u bytes (", uint32(sizeof(*MetadataSize)), uint32(*MetadataSize));
	const uint32 PrintByteCount = FMath::Min(32u, uint32(*MetadataSize));
	Metadata += sizeof(*MetadataSize);
	for (uint32 Index = 0; Index < PrintByteCount; ++Index, ++Metadata)
	{
		if (Index != 0)
		{
			UE_TRACE_ANALYSIS_DEBUG_AppendChar(' ');
		}
		UE_TRACE_ANALYSIS_DEBUG_Appendf("%02X", uint32(*Metadata));
	}
	if (PrintByteCount != uint32(*MetadataSize))
	{
		UE_TRACE_ANALYSIS_DEBUG_Append("...");
	}
	UE_TRACE_ANALYSIS_DEBUG_AppendChar(')');
	UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder();
#endif // UE_TRACE_ANALYSIS_DEBUG

	Reader.Advance(sizeof(*MetadataSize) + *MetadataSize);

	Context.Machine.QueueStage<FEstablishTransportStage>();
	Context.Machine.Transition();
	return EStatus::Continue;
}



// {{{1 magic ------------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FMagicStage
	: public FAnalysisMachine::FStage
{
public:
	virtual EStatus OnData(FStreamReader& Reader, const FMachineContext& Context) override;
};

////////////////////////////////////////////////////////////////////////////////
FMagicStage::EStatus FMagicStage::OnData(
	FStreamReader& Reader,
	const FMachineContext& Context)
{
	const auto* MagicPtr = Reader.GetPointer<uint32>();
	if (MagicPtr == nullptr)
	{
		return EStatus::NotEnoughData;
	}

	uint32 Magic = *MagicPtr;

	if (Magic == 'ECRT' || Magic == '2CRT')
	{
		// Source is big-endian which we don't currently support
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Invalid magic header (big-endian not supported)");
		Context.EmitMessage(EAnalysisMessageSeverity::Error, TEXT("Big endian traces are currently not supported."));
		return EStatus::Error;
	}

	if (Magic == 'TRCE')
	{
		Reader.Advance(sizeof(*MagicPtr));
		Context.Machine.QueueStage<FEstablishTransportStage>();
		Context.Machine.Transition();
		return EStatus::Continue;
	}

	if (Magic == 'TRC2')
	{
		Reader.Advance(sizeof(*MagicPtr));
		Context.Machine.QueueStage<FMetadataStage>();
		Context.Machine.Transition();
		return EStatus::Continue;
	}

	// There was no header on early traces so they went straight into declaring
	// protocol and transport versions.
	if (Magic == 0x00'00'00'01) // protocol 0, transport 1
	{
		Context.Machine.QueueStage<FEstablishTransportStage>();
		Context.Machine.Transition();
		return EStatus::Continue;
	}

	UE_TRACE_ANALYSIS_DEBUG_LOG("Error: Invalid magic header");
	Context.EmitMessage(EAnalysisMessageSeverity::Error, TEXT("The file or stream was not recognized as trace stream."));
	return EStatus::Error;
}



// {{{1 engine -----------------------------------------------------------------

////////////////////////////////////////////////////////////////////////////////
class FAnalysisEngine::FImpl
{
public:
						FImpl(TArray<IAnalyzer*>&& Analyzers, FMessageDelegate&& InMessage);
	void				Begin();
	void				End();
	bool				OnData(FStreamReader& Reader);
	FAnalysisBridge		Bridge;
	FAnalysisMachine	Machine;
};

////////////////////////////////////////////////////////////////////////////////
FAnalysisEngine::FImpl::FImpl(TArray<IAnalyzer*>&& Analyzers, FMessageDelegate&& InMessage)
: Bridge(Forward<TArray<IAnalyzer*>>(Analyzers))
, Machine(Bridge, Forward<FMessageDelegate>(InMessage))
{
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisEngine::FImpl::Begin()
{
	UE_TRACE_ANALYSIS_DEBUG_LOG("FAnalysisEngine::Begin()");

	Machine.QueueStage<FMagicStage>();
	Machine.Transition();
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisEngine::FImpl::End()
{
	Machine.Transition();

#if UE_TRACE_ANALYSIS_DEBUG
	FAnalysisState& State = Bridge.GetState();

	UE_TRACE_ANALYSIS_DEBUG_LOG("");
	UE_TRACE_ANALYSIS_DEBUG_LOG("SerialWrappedCount: %d", State.SerialWrappedCount);
	UE_TRACE_ANALYSIS_DEBUG_LOG("Serial.Value: %u (0x%X)", State.Serial.Value, State.Serial.Value);
	UE_TRACE_ANALYSIS_DEBUG_LOG("MaxEventDescs: %d", State.MaxEventDescs);
	UE_TRACE_ANALYSIS_DEBUG_LOG("SkippedSerialGaps: %d", State.NumSkippedSerialGaps);
	UE_TRACE_ANALYSIS_DEBUG_LOG("SkippedSerials: %llu", State.NumSkippedSerials);

	uint32 Digits = 0;
	uint64 Value = State.TotalEventSize;
	do
	{
		Value /= 10;
		++Digits;
	} while (Value != 0);

	uint64 ScopeEventCount = State.EnterScopeEventCount + State.LeaveScopeEventCount;
	uint64 ScopeTEventCount = State.EnterScopeTEventCount + State.LeaveScopeTEventCount;

	const int32 NC = Digits;
	UE_TRACE_ANALYSIS_DEBUG_LOG("");
	UE_TRACE_ANALYSIS_DEBUG_LOG("TotalEventCount:%*llu events", NC, State.TotalEventCount);
	if (State.TotalEventCount > 0)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("       NewEvent:%*llu events (%.1f%%)", NC, State.NewEventCount, (double)State.NewEventCount / (double)State.TotalEventCount * 100.0);
		UE_TRACE_ANALYSIS_DEBUG_LOG("           Sync:%*llu events (%.1f%%)", NC, State.SyncEventCount, (double)State.SyncEventCount / (double)State.TotalEventCount * 100.0);
		UE_TRACE_ANALYSIS_DEBUG_LOG("ImportantNoSync:%*llu events (%.1f%%)", NC, State.ImportantNoSyncEventCount, (double)State.ImportantNoSyncEventCount / (double)State.TotalEventCount * 100.0);
		UE_TRACE_ANALYSIS_DEBUG_LOG("    OtherNoSync:%*llu events (%.1f%%)", NC, State.OtherNoSyncEventCount, (double)State.OtherNoSyncEventCount / (double)State.TotalEventCount * 100.0);
		UE_TRACE_ANALYSIS_DEBUG_LOG("          Scope:%*llu events (%.1f%%) = %llu enter + %llu leave",
			NC, ScopeEventCount,
			(double)ScopeEventCount / (double)State.TotalEventCount * 100.0,
			State.EnterScopeEventCount,
			State.LeaveScopeEventCount);
		UE_TRACE_ANALYSIS_DEBUG_LOG("        Scope_T:%*llu events (%.1f%%) = %llu enter + %llu leave",
			NC, ScopeTEventCount,
			(double)ScopeTEventCount / (double)State.TotalEventCount * 100.0,
			State.EnterScopeTEventCount,
			State.LeaveScopeTEventCount);
		const int64 CountError =
			State.TotalEventCount
			- State.NewEventCount
			- State.SyncEventCount
			- State.ImportantNoSyncEventCount
			- State.OtherNoSyncEventCount
			- ScopeEventCount
			- ScopeTEventCount;
		if (CountError != 0)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("          error: %lld !!!", CountError);
		}
		uint64 DispatchedEventCount = State.TotalEventCount - State.NewEventCount - State.EnterScopeEventCount - State.EnterScopeTEventCount;
		uint64 NormalEventCount = State.SyncEventCount + State.ImportantNoSyncEventCount + State.OtherNoSyncEventCount - State.EnterScopeEventCount - State.EnterScopeTEventCount;
		check(DispatchedEventCount == NormalEventCount + ScopeEventCount + ScopeTEventCount);
		UE_TRACE_ANALYSIS_DEBUG_LOG("     Dispatched:%*llu events = %llu normal + %llu scoped", NC, DispatchedEventCount, NormalEventCount, ScopeEventCount + ScopeTEventCount);
		UE_TRACE_ANALYSIS_DEBUG_LOG("(normal dispatched events = sync events + no sync events - scope enter events)");
		UE_TRACE_ANALYSIS_DEBUG_LOG("");
	}

	uint64 ScopeEventSize = State.EnterScopeEventSize + State.LeaveScopeEventSize;
	uint64 ScopeTEventSize = State.EnterScopeTEventSize + State.LeaveScopeTEventSize;

	const int32 NS = Digits + 1;
	UE_TRACE_ANALYSIS_DEBUG_LOG(" TotalEventSize:%*llu bytes (%.1f MiB)", NS, State.TotalEventSize, (double)State.TotalEventSize / (1024.0 * 1024.0));
	if (State.TotalEventSize > 0)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("       NewEvent:%*llu bytes (%.1f%%)", NS, State.NewEventSize, (double)State.NewEventSize / (double)State.TotalEventSize * 100.0);
		UE_TRACE_ANALYSIS_DEBUG_LOG("           Sync:%*llu bytes (%.1f%%)", NS, State.SyncEventSize, (double)State.SyncEventSize / (double)State.TotalEventSize * 100.0);
		UE_TRACE_ANALYSIS_DEBUG_LOG("ImportantNoSync:%*llu bytes (%.1f%%)", NS, State.ImportantNoSyncEventSize, (double)State.ImportantNoSyncEventSize / (double)State.TotalEventSize * 100.0);
		UE_TRACE_ANALYSIS_DEBUG_LOG("    OtherNoSync:%*llu bytes (%.1f%%)", NS, State.OtherNoSyncEventSize, (double)State.OtherNoSyncEventSize / (double)State.TotalEventSize * 100.0);
		UE_TRACE_ANALYSIS_DEBUG_LOG("          Scope:%*llu bytes (%.1f%%) = %llu enter + %llu leave",
			NS, ScopeEventSize,
			(double)ScopeEventSize / (double)State.TotalEventSize * 100.0,
			State.EnterScopeEventSize,
			State.LeaveScopeEventSize);
		UE_TRACE_ANALYSIS_DEBUG_LOG("        Scope_T:%*llu bytes (%.1f%%) = %llu enter + %llu leave",
			NS, ScopeTEventSize,
			(double)ScopeTEventSize / (double)State.TotalEventSize * 100.0,
			State.EnterScopeTEventSize,
			State.LeaveScopeTEventSize);
		const int64 SizeError =
			State.TotalEventSize
			- State.NewEventSize
			- State.SyncEventSize
			- State.ImportantNoSyncEventSize
			- State.OtherNoSyncEventSize
			- ScopeEventSize
			- ScopeTEventSize;
		if (SizeError != 0)
		{
			UE_TRACE_ANALYSIS_DEBUG_LOG("          error: %lld !!!", SizeError);
		}
		UE_TRACE_ANALYSIS_DEBUG_LOG("     Dispatched:%*llu bytes (Total - NewEvent)", NS, State.TotalEventSize - State.NewEventSize);
	}
#endif // UE_TRACE_ANALYSIS_DEBUG

	Bridge.Reset();

	UE_TRACE_ANALYSIS_DEBUG_LOG("");
	UE_TRACE_ANALYSIS_DEBUG_LOG("FAnalysisEngine::End()");
}

////////////////////////////////////////////////////////////////////////////////
bool FAnalysisEngine::FImpl::OnData(FStreamReader& Reader)
{
	bool bRet = (Machine.OnData(Reader) != FAnalysisMachine::EStatus::Error);
	bRet &= Bridge.IsStillAnalyzing();
	return bRet;
}



////////////////////////////////////////////////////////////////////////////////
FAnalysisEngine::FAnalysisEngine(TArray<IAnalyzer*>&& Analyzers, FMessageDelegate&& InMessage)
: Impl(new FImpl(Forward<TArray<IAnalyzer*>>(Analyzers), Forward<FMessageDelegate>(InMessage)))
{
}

////////////////////////////////////////////////////////////////////////////////
FAnalysisEngine::~FAnalysisEngine()
{
	delete Impl;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisEngine::Begin()
{
	Impl->Begin();
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisEngine::End()
{
	Impl->End();
}

////////////////////////////////////////////////////////////////////////////////
bool FAnalysisEngine::OnData(FStreamReader& Reader)
{
	return Impl->OnData(Reader);
}

  // }}}
} // namespace Trace
} // namespace UE

#undef LOCTEXT_NAMESPACE

/* vim: set foldlevel=1 : */
