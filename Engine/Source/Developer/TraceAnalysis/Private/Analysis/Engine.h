// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Trace/Analysis.h"

class FMessageLog;

namespace UE {
namespace Trace {

class IAnalyzer;
class FStreamReader;

////////////////////////////////////////////////////////////////////////////////
class FAnalysisEngine
{
public:
						FAnalysisEngine(TArray<IAnalyzer*>&& Analyzers, FMessageDelegate&& InMessage);
						~FAnalysisEngine();
	void				Begin();
	void				End();
	bool				OnData(FStreamReader& Reader);

private:
	class FImpl;
						FAnalysisEngine(const FAnalysisEngine&) = delete;
						FAnalysisEngine(const FAnalysisEngine&&) = delete;
	FAnalysisEngine		operator = (const FAnalysisEngine&) = delete;
	FAnalysisEngine		operator = (const FAnalysisEngine&&) = delete;
	FImpl*				Impl;
};

} // namespace Trace
} // namespace UE
