// Copyright Epic Games, Inc. All Rights Reserved.

#include "CborWriter.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/UniquePtr.h"
#include "Trace/Analyzer.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
template<typename DestType>
static void WriteArray(FCborWriter& CborWriter, const IAnalyzer::TArrayReader<DestType>& Reader)
{
	if (const uint32 Num = Reader.Num())
	{
		CborWriter.WriteContainerStart(ECborCode::Array, Reader.Num());
		for (uint32 j = 0; j < Num; ++j)
		{
			DestType Value = Reader[j];
			CborWriter.WriteValue(Value);
		}
	}
	else
	{
		CborWriter.WriteNull();
	}
}
	
////////////////////////////////////////////////////////////////////////////////
void SerializeToCborImpl(
	TArray<uint8>& Out,
	const IAnalyzer::FEventData& EventData,
	uint32 EventSize)
{
	/* All this look up of fields is a bit slow. It'd be better to use the internals
	 * of the analysis engine instead */

	const IAnalyzer::FEventTypeInfo& TypeInfo = EventData.GetTypeInfo();
	const uint32 FieldCount = TypeInfo.GetFieldCount();
	if (FieldCount == 0)
	{
		return;
	}

	uint32 SizeHint = ((EventSize * 11) / 10); // + 10%
	SizeHint += FieldCount * 16;
	SizeHint += Out.Num();

	Out.Reserve(SizeHint);
	FMemoryWriter MemoryWriter(Out, false, true);
	FCborWriter CborWriter(&MemoryWriter, ECborEndianness::StandardCompliant);

	CborWriter.WriteContainerStart(ECborCode::Map, FieldCount);
	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		const IAnalyzer::FEventFieldInfo& FieldInfo = *(TypeInfo.GetFieldInfo(FieldIndex));

		const ANSICHAR* FieldName = FieldInfo.GetName();
		CborWriter.WriteValue(FieldName, FCStringAnsi::Strlen(FieldName));

		if (FieldInfo.GetType() == IAnalyzer::FEventFieldInfo::EType::AnsiString)
		{
			FAnsiStringView View;
			EventData.GetString(FieldName, View);
			CborWriter.WriteValue((const char*)(View.GetData()), View.Len());
			continue;
		}

		if (FieldInfo.GetType() == IAnalyzer::FEventFieldInfo::EType::WideString)
		{
			FString String;
			EventData.GetString(FieldName, String);
			CborWriter.WriteValue(String);
			continue;
		}

		if (FieldInfo.IsArray())
		{
			switch (FieldInfo.GetType())
			{
			case IAnalyzer::FEventFieldInfo::EType::Integer:
				WriteArray(CborWriter, EventData.GetArray<uint64>(FieldName));
				break;
			case IAnalyzer::FEventFieldInfo::EType::Float:
				WriteArray(CborWriter, EventData.GetArray<double>(FieldName));
				break;
			default:
				//not supported
				CborWriter.WriteNull();
				break;
			}
			continue;
		}

		if (FieldInfo.GetType() == IAnalyzer::FEventFieldInfo::EType::Integer)
		{
			if (FieldInfo.IsSigned())
			{
				int64 Value = EventData.GetValue<int64>(FieldName);
				CborWriter.WriteValue(Value);
			}
			else
			{
				uint64 Value = EventData.GetValue<uint64>(FieldName);
				CborWriter.WriteValue(Value);
			}
			continue;
		}

		if (FieldInfo.GetType() == IAnalyzer::FEventFieldInfo::EType::Float)
		{
			double Value = EventData.GetValue<double>(FieldName);
			CborWriter.WriteValue(Value);
			continue;
		}

		// No suitable value type was added if we get here.
		CborWriter.WriteNull();
	}
}

} // namespace Trace
} // namespace UE
