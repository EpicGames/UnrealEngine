// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processor.h"
#include "HAL/Event.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "StreamReader.h"
#include "TraceAnalysisModule.h"
#include "Logging/MessageLog.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/DataStream.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
FAnalysisProcessor::FImpl::FImpl(IInDataStream& InDataStream, TArray<IAnalyzer*>&& InAnalyzers, FMessageDelegate&& InMessage, FOnCompletedDelegate&& InOnCompleted)
: AnalysisEngine(Forward<TArray<IAnalyzer*>>(InAnalyzers), Forward<FMessageDelegate>(InMessage))
, DataStream(InDataStream)
, StopEvent(FPlatformProcess::GetSynchEventFromPool(true))
, UnpausedEvent(FPlatformProcess::GetSynchEventFromPool(true))
, OnCompleted(MoveTemp(InOnCompleted))
{
	Thread = FRunnableThread::Create(this, TEXT("TraceAnalysis"));
	PauseAnalysis(false);
}

////////////////////////////////////////////////////////////////////////////////
FAnalysisProcessor::FImpl::~FImpl()
{
	StopAnalysis();
	FPlatformProcess::ReturnSynchEventToPool(UnpausedEvent);
	FPlatformProcess::ReturnSynchEventToPool(StopEvent);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FAnalysisProcessor::FImpl::Run()
{
	LLM_SCOPE_BYNAME(TEXT("TraceAnalysis"));

	// Allow async streams to establish the data connection before
	// starting the read loop.
	while (!StopEvent->Wait(0, true) && !DataStream.WaitUntilReady())
	{}

	AnalysisEngine.Begin();

	FStreamBuffer Buffer(4 << 20);
	while (!StopEvent->Wait(0, true))
	{
		UnpausedEvent->Wait();

		int32 BytesRead = Buffer.Fill([&] (uint8* Out, uint32 Size)
		{
			return DataStream.Read(Out, Size);
		});

		if (BytesRead <= 0)
		{
			break;
		}

		if (!AnalysisEngine.OnData(Buffer))
		{
			break;
		}
	}

	AnalysisEngine.End();
	bComplete = true;
	OnCompleted.ExecuteIfBound();
	return 0;
}

////////////////////////////////////////////////////////////////////////////////
bool FAnalysisProcessor::FImpl::IsActive() const
{
	return !bComplete;
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisProcessor::FImpl::StopAnalysis()
{
	if (IsActive())
	{
		StopEvent->Trigger();
		WaitOnAnalysis();
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisProcessor::FImpl::WaitOnAnalysis()
{
	if (IsActive())
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FAnalysisProcessor::FImpl::PauseAnalysis(bool bState)
{
	if (IsActive())
	{
		bState ? UnpausedEvent->Reset() : UnpausedEvent->Trigger();
	}
}


////////////////////////////////////////////////////////////////////////////////
bool FAnalysisProcessor::IsActive() const	{ return (Impl != nullptr) ? Impl->IsActive() : false; }
void FAnalysisProcessor::Stop()				{ if (Impl != nullptr) { Impl->StopAnalysis(); } }
void FAnalysisProcessor::Wait()				{ if (Impl != nullptr) { Impl->WaitOnAnalysis(); } }
void FAnalysisProcessor::Pause(bool bState) { if (Impl != nullptr) { Impl->PauseAnalysis(bState); } }

////////////////////////////////////////////////////////////////////////////////
FAnalysisProcessor::FAnalysisProcessor(FAnalysisProcessor&& Rhs)
{
	this->operator = (MoveTemp(Rhs));
}

////////////////////////////////////////////////////////////////////////////////
FAnalysisProcessor& FAnalysisProcessor::operator = (FAnalysisProcessor&& Rhs)
{
	Swap(Impl, Rhs.Impl);
	return *this;
}

////////////////////////////////////////////////////////////////////////////////
FAnalysisProcessor::~FAnalysisProcessor()
{
	delete Impl;
}

} // namespace Trace
} // namespace UE
