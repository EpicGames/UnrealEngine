// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analysis/Engine.h"
#include "HAL/Runnable.h"
#include "Logging/MessageLog.h"
#include "Trace/Analysis.h"

class FEvent;
class FRunnableThread;

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
class FAnalysisProcessor::FImpl
	: public FRunnable
{
public:
						FImpl(IInDataStream& DataStream, TArray<IAnalyzer*>&& InAnalyzers, FMessageDelegate&& InMessage, FOnCompletedDelegate&& InOnCompleted);
						~FImpl();
	virtual uint32		Run() override;
	bool				IsActive() const;
	void				StopAnalysis();
	void				WaitOnAnalysis();
	void				PauseAnalysis(bool bState);
	FMessageLog*		GetLog();

private:
	FAnalysisEngine		AnalysisEngine;
	IInDataStream&		DataStream;
	FEvent*				StopEvent;
	FEvent*				UnpausedEvent;
	FRunnableThread*	Thread = nullptr;
	volatile bool		bComplete = false;
	FOnCompletedDelegate OnCompleted;
};

} // namespace Trace
} // namespace UE
