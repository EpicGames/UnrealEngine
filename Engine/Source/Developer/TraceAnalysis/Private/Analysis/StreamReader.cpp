// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamReader.h"
#include "Math/UnrealMath.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
const uint8* FStreamReader::GetPointer(uint32 Size) const
{
	if (Cursor + Size > End)
	{
		DemandHint = FMath::Max(DemandHint, Size);
		return nullptr;
	}

	return Buffer + Cursor;
}

////////////////////////////////////////////////////////////////////////////////
void FStreamReader::Advance(uint32 Size)
{
	Cursor += Size;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStreamReader::GetRemaining() const
{
	check(End >= Cursor);
	return End - Cursor;
}

////////////////////////////////////////////////////////////////////////////////
bool FStreamReader::CanMeetDemand() const
{
	return GetRemaining() >= DemandHint;
}

////////////////////////////////////////////////////////////////////////////////
bool FStreamReader::IsEmpty() const
{
	return Cursor >= End;
}

////////////////////////////////////////////////////////////////////////////////
bool FStreamReader::Backtrack(const uint8* To)
{
	uint32 BacktrackedCursor = uint32(UPTRINT(To - Buffer));
	if (BacktrackedCursor > End)
	{
		return false;
	}

	Cursor = BacktrackedCursor;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
struct FMark* FStreamReader::SaveMark() const
{
	return (FMark*)(UPTRINT(Cursor));
}

////////////////////////////////////////////////////////////////////////////////
void FStreamReader::RestoreMark(struct FMark* Mark)
{
	Cursor = uint32(UPTRINT(Mark));
	if (Cursor > End)
	{
		Cursor = End;
	}
}



////////////////////////////////////////////////////////////////////////////////
FStreamBuffer::FStreamBuffer(uint32 InitialBufferSize)
: BufferSize(InitialBufferSize)
{
	Buffer = (uint8*)FMemory::Malloc(BufferSize);
}

////////////////////////////////////////////////////////////////////////////////
void FStreamBuffer::Append(const uint8* Data, uint32 Size)
{
	uint8* Out = Append(Size);
	FMemory::Memcpy(Out, Data, Size);
}

////////////////////////////////////////////////////////////////////////////////
uint8* FStreamBuffer::Append(uint32 Size)
{
	DemandHint = FMath::Max(Size, DemandHint);
	Consolidate();
	uint8* Out = Buffer + End;
	End += Size;
	check(End <= BufferSize);
	return Out;
}

////////////////////////////////////////////////////////////////////////////////
void FStreamBuffer::Consolidate()
{
	uint32 Remaining = GetRemaining();
	check((uint64)DemandHint + (uint64)Remaining < (1ull << 31));
	DemandHint += Remaining;

	if (DemandHint >= BufferSize)
	{
		check(DemandHint < (1u << 31));
		BufferSize = FMath::Max(64u << 10, FMath::RoundUpToPowerOfTwo(DemandHint + 1));
		Buffer = (uint8*)FMemory::Realloc(Buffer, BufferSize);
	}

	if (!Remaining)
	{
		Cursor = 0;
		End = 0;
	}
	else if (Cursor)
	{
		memmove(Buffer, Buffer + Cursor, Remaining);
		Cursor = 0;
		End = Remaining;
		check(End <= BufferSize);
	}

	DemandHint = 0;
}

} // namespace Trace
} // namespace UE
