// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
class FStreamReader
{
public:
	template <typename Type>
	Type const*					GetPointer() const;
	template <typename Type>
	Type const*					GetPointerUnchecked() const;
	const uint8*				GetPointer(uint32 Size) const;
	const uint8*				GetPointerUnchecked() const;
	void						Advance(uint32 Size);
	bool						IsEmpty() const;
	bool						CanMeetDemand() const;
	uint32						GetRemaining() const;
	uint32						GetBacktrackSize(const uint8* To) const;
	bool						Backtrack(const uint8* To);
	struct FMark*				SaveMark() const;
	void						RestoreMark(struct FMark* Mark);

protected:
	uint8*						Buffer = nullptr;
	mutable uint32				DemandHint = 0;
	uint32						Cursor = 0;
	uint32						End = 0;
};

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
Type const* FStreamReader::GetPointer() const
{
	return (Type const*)GetPointer(sizeof(Type));
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type>
Type const* FStreamReader::GetPointerUnchecked() const
{
	return (Type const*)GetPointerUnchecked();
}

////////////////////////////////////////////////////////////////////////////////
inline const uint8* FStreamReader::GetPointerUnchecked() const
{
	return Buffer + Cursor;
}
////////////////////////////////////////////////////////////////////////////////
inline uint32 FStreamReader::GetBacktrackSize(const uint8* To) const
{
	return End - uint32(UPTRINT(To - Buffer));
}



////////////////////////////////////////////////////////////////////////////////
class FStreamBuffer
	: public FStreamReader
{
public:
								FStreamBuffer() = default;
								FStreamBuffer(uint32_t InitalBufferSize);
								~FStreamBuffer();
								FStreamBuffer(FStreamBuffer&& Rhs) noexcept;
								FStreamBuffer(const FStreamBuffer&)	= default;
	FStreamBuffer&				operator = (FStreamBuffer&& Rhs) noexcept;
	FStreamBuffer&				operator = (const FStreamBuffer&)	= delete;
	template <typename Lambda>
	int32						Fill(Lambda&& Source);
	void						Append(const uint8* Data, uint32 Size);
	uint8*						Append(uint32 Size);
	uint32						GetBufferSize() const { return BufferSize; }

protected:
	void						Consolidate();
	uint32						BufferSize = 0;
};

////////////////////////////////////////////////////////////////////////////////
inline FStreamBuffer::~FStreamBuffer()
{
	FMemory::Free(Buffer);
}

////////////////////////////////////////////////////////////////////////////////
inline FStreamBuffer::FStreamBuffer(FStreamBuffer&& Rhs) noexcept
{
	Swap(BufferSize, Rhs.BufferSize);
	Swap(Buffer, Rhs.Buffer);
	Swap(Cursor, Rhs.Cursor);
	Swap(End, Rhs.End);
	Swap(DemandHint, Rhs.DemandHint);
}

////////////////////////////////////////////////////////////////////////////////
inline FStreamBuffer& FStreamBuffer::operator = (FStreamBuffer&& Rhs) noexcept
{
	Swap(BufferSize, Rhs.BufferSize);
	Swap(Buffer, Rhs.Buffer);
	Swap(Cursor, Rhs.Cursor);
	Swap(End, Rhs.End);
	Swap(DemandHint, Rhs.DemandHint);
	return *this;
}

////////////////////////////////////////////////////////////////////////////////
template <typename Lambda>
inline int32 FStreamBuffer::Fill(Lambda&& Source)
{
	Consolidate();

	uint8* Dest = Buffer + End;
	int32 ReadSize = Source(Dest, BufferSize - End);
	if (ReadSize > 0)
	{
		End += ReadSize;
	}

	return ReadSize;
}

} // namespace Trace
} // namespace UE
