// Copyright Epic Games, Inc. All Rights Reserved.

#include "PacketTransport.h"
#include "HAL/UnrealMemory.h"

#include "TraceAnalysisDebug.h"

#include <initializer_list>

////////////////////////////////////////////////////////////////////////////////
namespace UE {
namespace Trace {
namespace Private {

TRACELOG_API int32 Decode(const void*, int32, void*, int32);

} // namespace Private
} // namespace Trace
} // namespace UE



namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
struct FPacketTransport::FPacketNode
{
	FPacketNode*		Next;
	uint32				Cursor;
	uint16				Serial;
	uint16				Size;
	uint8				Data[];
};



////////////////////////////////////////////////////////////////////////////////
FPacketTransport::~FPacketTransport()
{
	for (FPacketNode* Root : {ActiveList, PendingList, FreeList})
	{
		for (FPacketNode* Node = Root; Node != nullptr;)
		{
			FPacketNode* Next = Node->Next;
			FMemory::Free(Node);
			Node = Next;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FPacketTransport::Advance(uint32 BlockSize)
{
	if (ActiveList != nullptr)
	{
		ActiveList->Cursor += BlockSize;
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FPacketTransport::IsEmpty() const
{
	return (ActiveList == nullptr) || (ActiveList->Cursor + 1 > ActiveList->Size);
}

////////////////////////////////////////////////////////////////////////////////
const uint8* FPacketTransport::GetPointerImpl(uint32 BlockSize)
{
	if (ActiveList == nullptr && !GetNextBatch())
	{
		return nullptr;
	}

	uint32 NextCursor = ActiveList->Cursor + BlockSize;
	if (NextCursor > ActiveList->Size)
	{
		FPacketNode* Node = ActiveList;
		ActiveList = ActiveList->Next;
		Node->Next = FreeList;
		FreeList = Node;
		return GetPointerImpl(BlockSize);
	}

	return ActiveList->Data + ActiveList->Cursor;
}

////////////////////////////////////////////////////////////////////////////////
FPacketTransport::FPacketNode* FPacketTransport::AllocateNode()
{
	FPacketNode* Node;
	if (FreeList != nullptr)
	{
		Node = FreeList;
		FreeList = Node->Next;
	}
	else
	{
		Node = (FPacketNode*)FMemory::Malloc(sizeof(FPacketNode) + MaxPacketSize);
	}

	Node->Cursor = 0;
	return Node;
}

////////////////////////////////////////////////////////////////////////////////
bool FPacketTransport::GetNextBatch()
{
	int16 LastSerial = -1;
	if (PendingList != nullptr)
	{
		LastSerial = PendingList->Serial;
	}

	while (true)
	{
		struct FPacketBase
		{
			uint16	Serial;
			uint16	PacketSize;
		};

		const auto* PacketBase = (const FPacketBase*)FTransport::GetPointerImpl(sizeof(FPacketBase));
		if (PacketBase == nullptr)
		{
			return false;
		}

		// If this new payload is part of the next event batch then we've finished
		// building the current batch. The current batch can be activated.
		int16 PacketSerial = (PacketBase->Serial & 0x7fff);
		if (LastSerial >= PacketSerial)
		{
			ActiveList = PendingList;
			PendingList = nullptr;
			break;
		}

		if (FTransport::GetPointerImpl(PacketBase->PacketSize) == nullptr)
		{
			return false;
		}

		FTransport::Advance(PacketBase->PacketSize);

		LastSerial = PacketSerial;

		FPacketNode* Node = AllocateNode();
		Node->Serial = PacketSerial;
		Node->Next = PendingList;
		PendingList = Node;

		bool bEncoded = (PacketBase->Serial != PacketSerial);
		if (bEncoded)
		{
			struct FPacketEncoded
				: public FPacketBase
			{
				uint16	DecodedSize;
				uint8	Data[];
			};
			auto* PacketEncoded = (FPacketEncoded*)PacketBase;

			Node->Size = (uint16)UE::Trace::Private::Decode(
				PacketEncoded->Data,
				int32(PacketEncoded->PacketSize - sizeof(FPacketEncoded)),
				Node->Data,
				PacketEncoded->DecodedSize
			);
		}
		else
		{
			struct FPacketRaw
				: public FPacketBase
			{
				uint8 Data[];
			};
			auto* PacketRaw = (FPacketRaw*)PacketBase;

			Node->Size = uint16(PacketBase->PacketSize - sizeof(FPacketBase));
			FMemory::Memcpy(Node->Data, PacketRaw->Data, Node->Size);
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FPacketTransport::DebugBegin()
{
#if UE_TRACE_ANALYSIS_DEBUG
	UE_TRACE_ANALYSIS_DEBUG_LOG("FPacketTransport::DebugBegin()");
#endif // UE_TRACE_ANALYSIS_DEBUG
}

////////////////////////////////////////////////////////////////////////////////
void FPacketTransport::DebugEnd()
{
#if UE_TRACE_ANALYSIS_DEBUG
	if (!IsEmpty())
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: FPacketTransport is not empty!");
	}
	UE_TRACE_ANALYSIS_DEBUG_LOG("FPacketTransport::DebugEnd()");
#endif // UE_TRACE_ANALYSIS_DEBUG
}

////////////////////////////////////////////////////////////////////////////////
} // namespace Trace
} // namespace UE
