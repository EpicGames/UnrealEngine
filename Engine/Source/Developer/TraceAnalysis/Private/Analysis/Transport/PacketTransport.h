// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transport.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
class FPacketTransport
	: public FTransport
{
public:
							FPacketTransport() {}
	virtual					~FPacketTransport();
	virtual void			Advance(uint32 BlockSize) override;
	virtual bool			IsEmpty() const override;
	virtual void			DebugBegin() override;
	virtual void			DebugEnd() override;

protected:
	virtual const uint8*	GetPointerImpl(uint32 BlockSize) override;

private:
	struct					FPacketNode;
	bool					GetNextBatch();
	FPacketNode*			AllocateNode();
	static const uint32		MaxPacketSize = 8192;
	FPacketNode*			ActiveList = nullptr;
	FPacketNode*			PendingList = nullptr;
	FPacketNode*			FreeList = nullptr;
};

} // namespace Trace
} // namespace UE
