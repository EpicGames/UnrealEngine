// Copyright Epic Games, Inc. All Rights Reserved.

#include "TidPacketTransport.h"
#include "Algo/BinarySearch.h"
#include "HAL/UnrealMemory.h"
#include "CoreGlobals.h"

////////////////////////////////////////////////////////////////////////////////
namespace UE {
namespace Trace {
namespace Private {

TRACELOG_API int32 Decode(const void*, int32, void*, int32);

} // namespace Private
} // namespace Trace
} // namespace UE



namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
bool FTidPacketTransport::IsEmpty() const
{
	if (Threads.Num() > 0)
	{
		for (const FThreadStream& Thread : Threads)
		{
			if (!Thread.Buffer.IsEmpty())
			{
				return false;
			}
		}
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////
FTidPacketTransport::EReadPacketResult FTidPacketTransport::ReadPacket()
{
	using namespace UE::Trace::Private;

	const auto* PacketBase = GetPointer<FTidPacketBase>();
	if (PacketBase == nullptr)
	{
		return EReadPacketResult::NeedMoreData;
	}

	if (GetPointer<uint8>(PacketBase->PacketSize) == nullptr)
	{
		return EReadPacketResult::NeedMoreData;
	}

#if UE_TRACE_ANALYSIS_DEBUG
	++NumPackets;
	TotalPacketHeaderSize += sizeof(FTidPacketBase);
	TotalPacketSize += PacketBase->PacketSize;
#endif // UE_TRACE_ANALYSIS_DEBUG

	FTransport::Advance(PacketBase->PacketSize);

	uint32 ThreadId = PacketBase->ThreadId & FTidPacketBase::ThreadIdMask;

	if (ThreadId == ETransportTid::Sync)
	{
		++Synced;
#if UE_TRACE_ANALYSIS_DEBUG
		UE_TRACE_ANALYSIS_DEBUG_LOG("[SYNC %u]", Synced);
#endif // UE_TRACE_ANALYSIS_DEBUG
		return EReadPacketResult::NeedMoreData;	// Do not read any more packets. Gives consumers a
													// chance to sample the world at each known sync point.
	}

#if UE_TRACE_PACKET_VERIFICATION
	const bool bHasPacketSerial = !!(PacketBase->ThreadId & FTidPacketBase::Verification);
	FThreadStream* Thread = FindOrAddThread(ThreadId, true);
#else
	bool bIsPartial = !!(PacketBase->ThreadId & FTidPacketBase::PartialMarker);
	FThreadStream* Thread = FindOrAddThread(ThreadId, !bIsPartial);
#endif

	if (Thread == nullptr)
	{
		return EReadPacketResult::Continue;
	}

	uint32 DataSize = PacketBase->PacketSize - sizeof(FTidPacketBase);
	if (PacketBase->ThreadId & FTidPacketBase::EncodedMarker)
	{
		const auto* Packet = (const FTidPacketEncoded*)PacketBase;
		uint16 DecodedSize = Packet->DecodedSize;

#if UE_TRACE_ANALYSIS_DEBUG
		UE_TRACE_ANALYSIS_DEBUG_LOG_EX(3, "[PACKET %u] Tid=%u, Size: %u + %u bytes, DecodedSize: %u bytes (%.0f%%)%s",
			NumPackets,
			ThreadId,
			uint32(sizeof(FTidPacketEncoded)),
			DataSize,
			DecodedSize,
			((double)DataSize * 100.0) / (double)DecodedSize - 100.0,
			((DataSize == 0) || (DataSize > DecodedSize)) ? " !!!" : "");
		TotalPacketHeaderSize += sizeof(FTidPacketEncoded) - sizeof(FTidPacketBase);
		TotalDecodedSize += DecodedSize;
		uint64& TotalDataSizePerThread = DataSizePerThread.FindOrAdd(ThreadId);
		TotalDataSizePerThread += DecodedSize;
#endif // UE_TRACE_ANALYSIS_DEBUG

		DataSize -= sizeof(DecodedSize);

		// for debugging purposes only
		//UE_LOGF(LogCore, Log, "Decompress packet (tid=%u): %u --> %u bytes", ThreadId, DataSize, uint32(DecodedSize));

		if (DataSize != 0)
		{
			uint8* Dest = Thread->Buffer.Append(DecodedSize);
			int32 ResultSize = UE::Trace::Private::Decode(Packet->Data, DataSize, Dest, DecodedSize);
			if (int32(DecodedSize) != ResultSize)
			{
				UE_LOGF(LogCore, Error, "Unable to decompress packet; tid=%u, expected %u bytes, but decoded %d bytes!", ThreadId, uint32(DecodedSize), ResultSize);
				return EReadPacketResult::ReadError;
			}

#if UE_TRACE_ANALYSIS_DEBUG
			if (UE::Trace::Analysis::GDebugLevel >= 4)
			{
				UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder();
				for (uint32 i = 0; i < 32 && i < DecodedSize; ++i)
				{
					UE_TRACE_ANALYSIS_DEBUG_Appendf("%02X ", Dest[i]);
				}
				if (DecodedSize > 32)
				{
					UE_TRACE_ANALYSIS_DEBUG_Append("...");
				}
				UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder();
			}
#endif // UE_TRACE_ANALYSIS_DEBUG
		}
		else // DataSize == 0
		{
			UE_LOGF(LogCore, Warning, "Unable to decompress packet; tid=%u, expected to decompress %u bytes, but the packet is empty!", ThreadId, uint32(DecodedSize));
		}
	}
	else // not encoded
	{
#if UE_TRACE_ANALYSIS_DEBUG
		UE_TRACE_ANALYSIS_DEBUG_LOG_EX(3, "[PACKET %u] Tid=%u, Size: %u + %u bytes",
			NumPackets,
			ThreadId,
			uint32(sizeof(FTidPacket)),
			DataSize);
		TotalPacketHeaderSize += sizeof(FTidPacket) - sizeof(FTidPacketBase);
		TotalDecodedSize += DataSize;
		uint64& TotalDataSizePerThread = DataSizePerThread.FindOrAdd(ThreadId);
		TotalDataSizePerThread += DataSize;
#endif // UE_TRACE_ANALYSIS_DEBUG

		// for debugging purposes only
		//UE_LOGF(LogCore, Log, "Uncompressed packet (tid=%u): %u bytes", ThreadId, DataSize);

		Thread->Buffer.Append((uint8*)(PacketBase + 1), DataSize);

#if UE_TRACE_ANALYSIS_DEBUG
		if (UE::Trace::Analysis::GDebugLevel >= 4)
		{
			UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder();
			for (uint32 i = 0; i < 32 && i < DataSize; ++i)
			{
				UE_TRACE_ANALYSIS_DEBUG_Appendf("%02X ", ((uint8*)(PacketBase + 1))[i]);
			}
			if (DataSize > 32)
			{
				UE_TRACE_ANALYSIS_DEBUG_Append("...");
			}
			UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder();
		}
#endif // UE_TRACE_ANALYSIS_DEBUG
	}

#if UE_TRACE_PACKET_VERIFICATION
	if (bHasPacketSerial)
	{
		uint64 PacketSerial = *GetPointer<uint64>();
		if ((LastPacketSerial + 1) != PacketSerial)
		{
			UE_LOGF(LogCore, Error, "Found packet with index '%llu' when '%llu' was expected.", PacketSerial, LastPacketSerial);
			return EReadPacketResult::ReadError;
		}
		LastPacketSerial = PacketSerial;
		FTransport::Advance(sizeof(uint64));
	}
#endif

	return EReadPacketResult::Continue;
}

////////////////////////////////////////////////////////////////////////////////
FTidPacketTransport::FThreadStream* FTidPacketTransport::FindOrAddThread(
	uint32	ThreadId,
	bool	bAddIfNotFound)
{
	uint32 ThreadCount = Threads.Num();
	for (uint32 i = 0; i < ThreadCount; ++i)
	{
		if (Threads[i].ThreadId == ThreadId)
		{
			return &(Threads[i]);
		}
	}

	if (!bAddIfNotFound)
	{
		return nullptr;
	}

	FThreadStream Thread;
	Thread.ThreadId = ThreadId;
	Threads.Add(Thread);
	return &(Threads[ThreadCount]);
}

////////////////////////////////////////////////////////////////////////////////
FTidPacketTransport::ETransportResult FTidPacketTransport::Update()
{
	EReadPacketResult Result;
	do
	{
		Result = ReadPacket();
	} while (Result == EReadPacketResult::Continue);

	Threads.RemoveAll([] (const FThreadStream& Thread)
	{
		return Thread.ThreadId != ETransportTid::Events &&
				Thread.ThreadId != ETransportTid::Importants &&
				Thread.ThreadId != ETransportTid::PseudoImportants &&
				Thread.Buffer.IsEmpty();
	});

#if UE_TRACE_ANALYSIS_DEBUG
	DebugUpdate();
#endif // UE_TRACE_ANALYSIS_DEBUG

	return Result == EReadPacketResult::ReadError ? ETransportResult::Error : ETransportResult::Ok;
}

////////////////////////////////////////////////////////////////////////////////
void FTidPacketTransport::DebugUpdate()
{
#if UE_TRACE_ANALYSIS_DEBUG
	const int32 ThreadCount = Threads.Num();
	for (int32 ThreadIndex = 0; ThreadIndex < ThreadCount; ++ThreadIndex)
	{
		const FThreadStream& Thread = Threads[ThreadIndex];
		const uint32 BufferSize = Thread.Buffer.GetBufferSize();
		const uint32 DataSize = Thread.Buffer.GetRemaining();

		if (BufferSize > MaxBufferSize)
		{
			MaxBufferSize = BufferSize;
			MaxBufferSizeThreadId = Thread.ThreadId;
		}

		if (DataSize > MaxDataSizePerBuffer)
		{
			MaxDataSizePerBuffer = DataSize;
			MaxDataSizePerBufferThreadId = Thread.ThreadId;
		}
	}
#endif // UE_TRACE_ANALYSIS_DEBUG
}

////////////////////////////////////////////////////////////////////////////////
void FTidPacketTransport::DebugBegin()
{
#if UE_TRACE_ANALYSIS_DEBUG
	UE_TRACE_ANALYSIS_DEBUG_LOG("FTidPacketTransport::DebugBegin()");
#endif // UE_TRACE_ANALYSIS_DEBUG
}

////////////////////////////////////////////////////////////////////////////////
void FTidPacketTransport::DebugEnd()
{
#if UE_TRACE_ANALYSIS_DEBUG
	Threads.RemoveAll([] (const FThreadStream& Thread)
	{
		return Thread.Buffer.IsEmpty();
	});

	DebugUpdate();

	constexpr double MiB = 1024.0 * 1024.0;

	UE_TRACE_ANALYSIS_DEBUG_LOG("");
	UE_TRACE_ANALYSIS_DEBUG_LOG("TotalPacketSize: %llu bytes (%.1f MiB)", TotalPacketSize, (double)TotalPacketSize / MiB);
	const uint64 AdjustedTotalDecodedSize = TotalPacketHeaderSize + TotalDecodedSize;
	UE_TRACE_ANALYSIS_DEBUG_LOG("TotalDecodedSize: %llu bytes + %llu bytes = %llu bytes (%.1f MiB)", TotalPacketHeaderSize, TotalDecodedSize, AdjustedTotalDecodedSize, (double)AdjustedTotalDecodedSize / MiB);
	const int64 Saving = (int64)AdjustedTotalDecodedSize - (int64)TotalPacketSize;
	const double SavingPercent = ((double)TotalPacketSize * 100.0) / (double)AdjustedTotalDecodedSize - 100.0;
	UE_TRACE_ANALYSIS_DEBUG_LOG("Compression Savings: %lli bytes (%.1f MiB; %.2f%%)", Saving, (double)Saving / MiB, SavingPercent);

	UE_TRACE_ANALYSIS_DEBUG_LOG("NumPackets: %u", NumPackets);

	UE_TRACE_ANALYSIS_DEBUG_LOG("MaxBufferSize: %u bytes (%.1f MiB; for thread %u)", MaxBufferSize, (double)MaxBufferSize / MiB, MaxBufferSizeThreadId);
	const double MaxDataSizePerBufferPercent = ((double)MaxDataSizePerBuffer * 100.0) / (double)TotalDecodedSize;
	UE_TRACE_ANALYSIS_DEBUG_LOG("MaxDataSizePerBuffer: %u bytes (%.1f%%; for thread %u)", MaxDataSizePerBuffer, MaxDataSizePerBufferPercent, MaxDataSizePerBufferThreadId);

	uint64 MaxDataSizePerThread = 0;
	uint32 MaxDataSizePerThreadId = 0;
	for (auto& KV : DataSizePerThread)
	{
		if (KV.Value > MaxDataSizePerThread)
		{
			MaxDataSizePerThread = KV.Value;
			MaxDataSizePerThreadId = KV.Key;
		}
	}
	const double MaxDataSizePerThreadPercent = ((double)MaxDataSizePerThread * 100.0) / (double)TotalDecodedSize;
	UE_TRACE_ANALYSIS_DEBUG_LOG("MaxDataSizePerThread: %llu bytes (%.1f MiB; %.1f%%; for thread %u)", MaxDataSizePerThread, (double)MaxDataSizePerThread / MiB, MaxDataSizePerThreadPercent, MaxDataSizePerThreadId);

	const int32 ThreadCount = Threads.Num();
	UE_TRACE_ANALYSIS_DEBUG_LOG("Remaining Streaming Threads: %d", ThreadCount);
	uint64 TotalBufferSize = 0;
	uint64 UnprocessedDataSize = 0;
	for (int32 ThreadIndex = 0; ThreadIndex < ThreadCount; ++ThreadIndex)
	{
		const FThreadStream& Thread = Threads[ThreadIndex];
		const uint32 BufferSize = Thread.Buffer.GetBufferSize();
		const uint32 DataSize = Thread.Buffer.GetRemaining();
		const double DataSizePercent = ((double)DataSize * 100.0) / (double)TotalDecodedSize;
		UE_TRACE_ANALYSIS_DEBUG_LOG("[THREAD %d] Tid=%u BufferSize: %u bytes, DataSize: %u bytes (%.1f%%)", ThreadIndex, Thread.ThreadId, BufferSize, DataSize, DataSizePercent);
		TotalBufferSize += BufferSize;
		UnprocessedDataSize += DataSize;
	}
	UE_TRACE_ANALYSIS_DEBUG_LOG("TotalBufferSize: %llu bytes (%.1f MiB)", TotalBufferSize, (double)TotalBufferSize / MiB);

	const int64 ProcessedDataSize = (int64)TotalDecodedSize - (int64)UnprocessedDataSize;
	const double ProcessedPercent = ((double)ProcessedDataSize * 100.0) / (double)TotalDecodedSize;

	UE_TRACE_ANALYSIS_DEBUG_LOG("TotalDataSize (processed): %llu bytes (%.1f MiB; %.1f%%)", ProcessedDataSize, (double)ProcessedDataSize / MiB, ProcessedPercent);
	UE_TRACE_ANALYSIS_DEBUG_LOG("TotalDataSize (unprocessed): %llu bytes (%.1f MiB; %.1f%%)", UnprocessedDataSize, (double)UnprocessedDataSize / MiB, 100.0 - ProcessedPercent);

	const bool bIsEmpty = IsEmpty();
	check((UnprocessedDataSize == 0 && bIsEmpty) || (UnprocessedDataSize != 0 && !bIsEmpty));

	if (!bIsEmpty)
	{
		UE_TRACE_ANALYSIS_DEBUG_LOG("Error: FTidPacketTransport is not empty!");
	}

	UE_TRACE_ANALYSIS_DEBUG_LOG("FTidPacketTransport::DebugEnd()");
#endif // UE_TRACE_ANALYSIS_DEBUG
}

////////////////////////////////////////////////////////////////////////////////
uint32 FTidPacketTransport::GetThreadCount() const
{
	return uint32(Threads.Num());
}

////////////////////////////////////////////////////////////////////////////////
FStreamReader* FTidPacketTransport::GetThreadStream(uint32 Index)
{
	return &(Threads[Index].Buffer);
}

////////////////////////////////////////////////////////////////////////////////
FStreamReader* FTidPacketTransport::GetKnownStream(EThreadStreamIndex KnownThread)
{
	check(KnownThread < EThreadStreamIndex::RealThreadsStart);
	return GetThreadStream((uint32)KnownThread);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FTidPacketTransport::GetThreadId(uint32 Index) const
{
	return Threads[Index].ThreadId;
}

} // namespace Trace
} // namespace UE
