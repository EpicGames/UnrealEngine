// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transport.h"
#include "Analysis/StreamReader.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "TraceAnalysisDebug.h"
#include "Trace/Detail/Transport.h"

namespace UE {
namespace Trace {

class FStreamReader;

////////////////////////////////////////////////////////////////////////////////
class FTidPacketTransport
	: public FTransport
{
public:
	enum class ETransportResult : uint8
	{
		Ok,
		Error
	};

	/**
	 * The "pseudo threads" have a fixed
	 * index in the thread stream array.
	 */
	enum class EThreadStreamIndex : uint32
	{
		Events = 0,
		Importants = 1,
		PseudoImportants = 2,
		RealThreadsStart = 3,
	};


							FTidPacketTransport() {}
	virtual					~FTidPacketTransport() = default;

	virtual bool			IsEmpty() const override;
	virtual void			DebugBegin() override;
	virtual void			DebugEnd() override;

	ETransportResult		Update();
	uint32					GetThreadCount() const;
	FStreamReader*			GetThreadStream(uint32 Index);
	FStreamReader*			GetKnownStream(EThreadStreamIndex KnownThread);
	uint32					GetThreadId(uint32 Index) const;
	uint32					GetSyncCount() const;


protected:
	uint32					Synced = 0x7fff'ffff;
	uint32					NumPackets = 0;
#if UE_TRACE_PACKET_VERIFICATION
	uint64					LastPacketSerial = 0;
#endif

private:
	struct FThreadStream
	{
		FStreamBuffer		Buffer;
		uint32				ThreadId;
	};

	enum class EReadPacketResult : uint8
	{
		NeedMoreData,
		Continue,
		ReadError
	};

	EReadPacketResult		ReadPacket();
	FThreadStream*			FindOrAddThread(uint32 ThreadId, bool bAddIfNotFound);
	void					DebugUpdate();

	TArray<FThreadStream>	Threads = {
								{ {}, ETransportTid::Events },
								{ {}, ETransportTid::Importants },
								{ {}, ETransportTid::PseudoImportants }
							};

#if UE_TRACE_ANALYSIS_DEBUG
	uint64					TotalPacketHeaderSize = 0;
	uint64					TotalPacketSize = 0;
	uint64					TotalDecodedSize = 0;
	uint32					MaxBufferSize = 0;
	uint32					MaxBufferSizeThreadId = 0;
	uint32					MaxDataSizePerBuffer = 0;
	uint32					MaxDataSizePerBufferThreadId = 0;
	TMap<uint32, uint64>	DataSizePerThread;
#endif // UE_TRACE_ANALYSIS_DEBUG
};

////////////////////////////////////////////////////////////////////////////////
inline uint32 FTidPacketTransport::GetSyncCount() const
{
	return Synced;
}



////////////////////////////////////////////////////////////////////////////////
class FTidPacketTransportSync
	: public FTidPacketTransport
{
public:
	FTidPacketTransportSync()
	{
		Synced = 0;
	}
	virtual ~FTidPacketTransportSync() = default;
};

} // namespace Trace
} // namespace UE
