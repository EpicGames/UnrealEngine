// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analysis/StreamReader.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
class FTransport
{
public:
							FTransport() {}
	virtual					~FTransport() = default;
	void					SetReader(FStreamReader& InReader);
	FStreamReader*			GetReader() const { return Reader; }
	template <typename RetType>
	RetType const*			GetPointer();
	template <typename RetType>
	RetType const*			GetPointer(uint32 BlockSize);
	virtual void			Advance(uint32 BlockSize);
	virtual bool			IsEmpty() const { return true; }
	virtual void			DebugBegin() {}
	virtual void			DebugEnd() {}

protected:
	virtual const uint8*	GetPointerImpl(uint32 BlockSize);

	FStreamReader*			Reader = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
inline void FTransport::SetReader(FStreamReader& InReader)
{
	Reader = &InReader;
}

////////////////////////////////////////////////////////////////////////////////
template <typename RetType>
inline RetType const* FTransport::GetPointer()
{
	return GetPointer<RetType>(sizeof(RetType));
}

////////////////////////////////////////////////////////////////////////////////
template <typename RetType>
inline RetType const* FTransport::GetPointer(uint32 BlockSize)
{
	return (RetType const*)GetPointerImpl(BlockSize);
}

////////////////////////////////////////////////////////////////////////////////
inline void FTransport::Advance(uint32 BlockSize)
{
	Reader->Advance(BlockSize);
}

////////////////////////////////////////////////////////////////////////////////
inline const uint8* FTransport::GetPointerImpl(uint32 BlockSize)
{
	return Reader->GetPointer(BlockSize);
}

} // namespace Trace
} // namespace UE
