// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if 0
#if PLATFORM_WINDOWS
#	include "Windows/AllowWindowsPlatformTypes.h"
#	define _WINSOCK_DEPRECATED_NO_WARNINGS  
#	include <winsock2.h>
#	include <ws2tcpip.h>
#	include "Windows/HideWindowsPlatformTypes.h"
#	pragma comment(lib, "ws2_32.lib")
#endif

#define TRACE_LOG(Format, ...) UE_LOG(LogCore, Log, TEXT(Format), __VA_ARGS__)
#endif // 0
