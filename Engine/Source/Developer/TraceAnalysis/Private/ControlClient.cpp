// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/ControlClient.h"
#include "IPAddress.h"
#include "Misc/CString.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "AddressInfoTypes.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
FControlClient::~FControlClient()
{
    Disconnect();
}

////////////////////////////////////////////////////////////////////////////////
bool FControlClient::Connect(const TCHAR* Host, uint16 Port)
{
    if (IsConnected())
    {
        return false;
    }

	ISocketSubsystem* Sockets = ISocketSubsystem::Get();
	if (Sockets == nullptr)
	{
		return false;
	}
	
	TSharedPtr<FInternetAddr> Addr = Sockets->GetAddressFromString(Host);
	if (!Addr.IsValid() || !Addr->IsValid())
	{
		FAddressInfoResult GAIRequest = Sockets->GetAddressInfo(Host, nullptr, EAddressInfoFlags::Default, NAME_None);
		if (GAIRequest.ReturnCode != SE_NO_ERROR || GAIRequest.Results.Num() == 0)
		{
			return false;
		}
		
		Addr = GAIRequest.Results[0].Address;
	}
	Addr->SetPort(Port);
	return Connect(*Addr);
}

////////////////////////////////////////////////////////////////////////////////
bool FControlClient::Connect(const FInternetAddr& Address)
{
	if (IsConnected())
	{
		return false;
	}
	
	ISocketSubsystem* Sockets = ISocketSubsystem::Get();	
	if (Sockets == nullptr)
	{
		return false;
	}

	FSocket* ClientSocket = Sockets->CreateSocket(NAME_Stream, TEXT("TraceControlClient"), Address.GetProtocolType());
	if (ClientSocket == nullptr)
	{
		return false;
	}

	if (!ClientSocket->Connect(Address))
	{
		Sockets->DestroySocket(ClientSocket);
		return false;
	}

	ClientSocket->SetLinger();

    Socket = ClientSocket;
    return true;
}

////////////////////////////////////////////////////////////////////////////////
void FControlClient::Disconnect()
{
    if (!IsConnected())
    {
        return;
    }

	Socket->Shutdown(ESocketShutdownMode::ReadWrite);
	Socket->Close();

	ISocketSubsystem& Sockets = *(ISocketSubsystem::Get());
	Sockets.DestroySocket(Socket);
	Socket = nullptr;
}

////////////////////////////////////////////////////////////////////////////////
bool FControlClient::IsConnected() const
{
    return (Socket != nullptr);
}

////////////////////////////////////////////////////////////////////////////////
void FControlClient::SendSendTo(const TCHAR* Host)
{
    FormatAndSend(TEXT("SendTo %s"), Host);
}

////////////////////////////////////////////////////////////////////////////////
void FControlClient::SendWriteTo(const TCHAR* Path)
{
    FormatAndSend(TEXT("WriteTo %s"), Path);
}

////////////////////////////////////////////////////////////////////////////////
void FControlClient::SendStop()
{
    FormatAndSend(TEXT("Stop"));
}

////////////////////////////////////////////////////////////////////////////////
void FControlClient::SendToggleChannel(const TCHAR* Channels, bool bState /*= true*/)
{
	FormatAndSend(TEXT("ToggleChannels %s %d"), Channels, bState);
}

////////////////////////////////////////////////////////////////////////////////
void FControlClient::Send(const TCHAR* Command)
{
	int Length = FCString::Strlen(Command);
	Send((const uint8*)TCHAR_TO_ANSI(Command), Length);
}

////////////////////////////////////////////////////////////////////////////////
void FControlClient::FormatAndSend(const TCHAR* Format, ...)
{
    if (!IsConnected())
    {
        return;
    }

    TCHAR Buffer[512];
	va_list Args;
	va_start(Args, Format);
	int Length = FCString::GetVarArgs(Buffer, UE_ARRAY_COUNT(Buffer), Format, Args);
    if (Length > sizeof(Buffer))
    {
        Length = sizeof(Buffer);
    }
	va_end(Args);

	Send((const uint8*)TCHAR_TO_ANSI(Buffer), Length);
}

////////////////////////////////////////////////////////////////////////////////
void FControlClient::Send(const uint8* Data, int Length)
{
    int32 SentBytes = 0;
    if (!Socket->Send(Data, Length, SentBytes) || SentBytes != Length)
    {
        Disconnect();
		return;
    }

    if (!Socket->Send((const uint8*)"\n", 1, SentBytes) || SentBytes != 1)
    {
        Disconnect();
		return;
    }
}

} // namespace Trace
} // namespace UE
