// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataStream.h"
#include "DataStreamInternal.h"

#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Event.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Logging/LogCategory.h"
#include "Sockets.h"

#if PLATFORM_LINUX
#include <sys/socket.h>
#endif

DEFINE_LOG_CATEGORY(LogTraceData);

namespace UE {
namespace Trace {

//--------------------------------------------------------------------
// FFileDataStream
//--------------------------------------------------------------------

FFileDataStream::FFileDataStream()
	: Handle(nullptr)
	, Remaining(0)
{
}

FFileDataStream::~FFileDataStream()
{
}

bool FFileDataStream::Open(const TCHAR* Path, bool bAllowWrite)
{
	Handle.Reset(FPlatformFileManager::Get().GetPlatformFile().OpenRead(Path, bAllowWrite));
	if (Handle == nullptr)
	{
		return false;
	}
	Remaining = Handle->Size();
	return true;
}

int32 FFileDataStream::Read(void* Data, uint32 Size)
{
	if (Handle == nullptr)
	{
		return -1;
	}

	if (Remaining <= 0)
	{
		return 0;
	}

	if (Size > Remaining)
	{
		Size = static_cast<uint32>(Remaining);
	}
	Remaining -= Size;

	if (!Handle->Read((uint8*)Data, Size))
	{
		Close();
		return -1;
	}

	return Size;
}

void FFileDataStream::Close()
{
	Handle.Reset();
}


//--------------------------------------------------------------------
// FTraceDataStream
//--------------------------------------------------------------------

FTraceDataStream::FTraceDataStream(asio::ip::tcp::socket& InSocket)
: Socket(MoveTemp(InSocket))
{
	asio::socket_base::receive_buffer_size RecvBufferSize(4 << 20);
	Socket.set_option(RecvBufferSize);
}

FTraceDataStream::~FTraceDataStream()
{
	FTraceDataStream::Close();
}

bool FTraceDataStream::IsOpen() const
{
	return Socket.is_open();
}

void FTraceDataStream::Close()
{
	Socket.shutdown(asio::ip::tcp::socket::shutdown_receive);
	Socket.close();
}

int32 FTraceDataStream::Read(void* Dest, uint32 DestSize)
{
	auto Handle = Socket.native_handle();

	fd_set Fds;
	FD_ZERO(&Fds);
	FD_SET(Handle, &Fds);

	timeval Timeout;
	Timeout.tv_sec = 1;
	Timeout.tv_usec = 0;

	while (true)
	{
		fd_set ReadFds = Fds;
		int Ret = select((int)Handle + 1, &ReadFds, 0, 0, &Timeout);

		if (Ret < 0)
		{
			Close();
			return -1;
		}

		if (Ret == 0)
		{
			continue;
		}

		asio::error_code ErrorCode;
		size_t BytesRead = Socket.read_some(asio::buffer(Dest, DestSize), ErrorCode);
		if (ErrorCode)
		{
			Close();
			return -1;
		}

		return int32(BytesRead);
	}
}

//--------------------------------------------------------------------
// FDirectSocketContext
//--------------------------------------------------------------------

/**
* Small utility class to avoid exposing the asio types in the public
* header.
*/
struct FDirectSocketContext
{
	FDirectSocketContext()
		: Context(1)
		, Acceptor(Context)
	{}

	asio::io_context Context;
	asio::ip::tcp::acceptor Acceptor;
};

//--------------------------------------------------------------------
// FDirectSocketStream
//--------------------------------------------------------------------

FDirectSocketStream::FDirectSocketStream()
	: ConnectionEvent(FPlatformProcess::GetSynchEventFromPool(false))
{
	AsioContext = MakeUnique<FDirectSocketContext>();
}

FDirectSocketStream::FDirectSocketStream(TUniquePtr<FArchiveFileWriterGeneric>&& InFileWriter)
	: FileWriter(MoveTemp(InFileWriter)),
	ConnectionEvent(FPlatformProcess::GetSynchEventFromPool(false))

{
	AsioContext = MakeUnique<FDirectSocketContext>();
}

FDirectSocketStream::~FDirectSocketStream()
{
	if (AsioContext)
	{
#if PLATFORM_LINUX
		// Socket needs to be shutdown before closing the asio acceptor, otherwise ListeningThread would deadlock.
		shutdown(AsioContext->Acceptor.native_handle(), SHUT_RD);
#endif
		AsioContext->Acceptor.close();
	}

	if (ListeningThread)
	{
		ListeningThread->WaitForCompletion();
		ListeningThread.Reset();
	}

	InternalStream.Reset();
	AsioContext.Reset();
}

uint16 FDirectSocketStream::StartListening(uint16 Port)
{
	if (Port == 0)
	{
		Port = DefaultPort;
	}
	// Try to bind the default port. If that is busy move to next candidate
	int32 Attempts = 0;
	while (!CreateSocket(Port) && Attempts < MaxPortAttempts)
	{
		++Port;
		++Attempts;
	}
	if (Attempts >= MaxPortAttempts)
	{
		return 0;
	}

	// We cannot block on this thread or on the analysis thread (supports cancellation), create
	// a short-lived thread for the blocking accept call.
	ListeningThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, TEXT("FDirectSocketStreamListener")));
	UE_LOGF(LogTraceData, Log, "Started listening thread for direct trace connection on port %u", Port);

	return Port;
}

bool FDirectSocketStream::CreateSocket(uint16 Port)
{
	using asio::ip::tcp;

	asio::error_code ErrorCode;
	tcp::acceptor Acceptor(AsioContext->Context);

#if PLATFORM_WINDOWS
	DWORD Flags = WSA_FLAG_NO_HANDLE_INHERIT|WSA_FLAG_OVERLAPPED;
	SOCKET Socket = WSASocketW(AF_INET, SOCK_STREAM, IPPROTO_TCP, nullptr, 0, Flags);
	Acceptor.assign(tcp::v4(), Socket, ErrorCode);
#else
	Acceptor.open(tcp::v4(), ErrorCode);
#endif
	if (ErrorCode)
	{
		UE_LOGF(LogTraceData, Error, "Failed to open acceptor: '%ls'", ANSI_TO_TCHAR(ErrorCode.message().c_str()));
		return false;
	}

	// Reuse address to avoid locking up port
	asio::socket_base::reuse_address reuse_option(true);
	Acceptor.set_option(reuse_option);

	// Setup endpoint for port on any address
	tcp::endpoint Endpoint(tcp::v4(), uint16(Port));

	Acceptor.bind(Endpoint, ErrorCode);
	if (ErrorCode)
	{
		UE_LOGF(LogTraceData, Error, "Failed to bind socket on port %u: '%ls'", Port, ANSI_TO_TCHAR(ErrorCode.message().c_str()));
		return false;
	}

	Acceptor.listen(MaxQueuedConnections, ErrorCode);
	if (ErrorCode)
	{
		UE_LOGF(LogTraceData, Error, "Failed to listen on port %u: '%ls'", Port, ANSI_TO_TCHAR(ErrorCode.message().c_str()));
		return false;
	}

	AsioContext->Acceptor = MoveTemp(Acceptor);
	return true;
}

void FDirectSocketStream::Accept()
{
	using asio::ip::tcp;
	if (!AsioContext.IsValid())
	{
		return;
	}

	tcp::socket Socket(AsioContext->Context);
	asio::error_code ErrorCode;
	AsioContext->Acceptor.accept(Socket, ErrorCode);

	if (ErrorCode)
	{
		if (ErrorCode == asio::error::interrupted)
		{
			return;
		}
		UE_LOGF(LogTraceData, Error, "Failed accept sockets connection, error: %u.", ErrorCode.value());
		return;
	}

#if PLATFORM_WINDOWS
	SetHandleInformation(HANDLE(SOCKET(Socket.native_handle())), HANDLE_FLAG_INHERIT, 0);
#endif

	{
		const auto Address = Socket.remote_endpoint().address();
		TStringBuilder<16> RemoteAddress;
		RemoteAddress << ANSI_TO_TCHAR(Address.to_string().c_str());
		UE_LOGF(LogTraceData, Log, "Accepted direct trace connection from %ls", RemoteAddress.ToString());
	}

	AsioContext->Acceptor.close();
	InternalStream = MakeUnique<FTraceDataStream>(Socket);
	ConnectionEvent->Trigger();
}

bool FDirectSocketStream::BindToExistingConnection(FSocket&& ExistingConnectedSocket)
{
	using asio::ip::tcp;

	// Take over the existing connection by releasing it
	UPTRINT NativeSocket = ExistingConnectedSocket.ReleaseNativeSocket();
	tcp::socket AsioSocket(AsioContext->Context);

	// Assign it to our internal asio representation
	asio::error_code ErrorCode;
	AsioSocket.assign(tcp::v4(), NativeSocket, ErrorCode);

	if (ErrorCode)
	{
		UE_LOGF(LogTraceData, Error, "%s: Failed to bind ASIO socket from FSocket connection, error: %u.", __func__, ErrorCode.value());
		return false;
	}

	// mimic behavior that we just completed the connection
	InternalStream = MakeUnique<FTraceDataStream>(AsioSocket);
	ConnectionEvent->Trigger();

	return true;
}

uint32 FDirectSocketStream::Run()
{
	// Listening thread entry point. We currently only accept the first connection
	// so the thread will exit immediately after.
	Accept();
	return 0;
}

void FDirectSocketStream::Stop()
{
	if (AsioContext)
	{
		// Stopping the listening thread by closing the acceptor, which
		// will abort the accept operation.
		AsioContext->Acceptor.close();
	}
}

bool FDirectSocketStream::WaitUntilReady()
{
	return ConnectionEvent->Wait(0, true);
}

int32 FDirectSocketStream::Read(void* Data, uint32 Size)
{
	if (!InternalStream)
	{
		// Treat trying to read the stream before it's ready as an error
		return -1;
	}

	const int32 BytesRead = InternalStream->Read(Data, Size);

	if (BytesRead > 0 && FileWriter.IsValid() && !FileWriter->IsError())
	{
		FileWriter->Serialize(Data, BytesRead);
	}

	return BytesRead;
}

void FDirectSocketStream::Close()
{
	if (InternalStream)
	{
		InternalStream->Close();
	}

	if (FileWriter.IsValid())
	{
		FileWriter->Close();
		FileWriter.Reset();
	}
}

} // namespace Trace
} // namespace UE
