// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Trace/DataStream.h"
#include "Logging/LogCategory.h"
#include "Asio.h"

DECLARE_LOG_CATEGORY_EXTERN(LogTraceData, Log, Log);

namespace UE {
namespace Trace {
	
////////////////////////////////////////////////////////////////////////////////
class FTraceDataStream
	: public IInDataStream
{
public:
	FTraceDataStream(asio::ip::tcp::socket& InSocket);
	virtual					~FTraceDataStream();
	bool					IsOpen() const;
	virtual void			Close() override;
	virtual int32			Read(void* Dest, uint32 DestSize) override;

private:
	asio::ip::tcp::socket	Socket;
};

	
} // namespace Trace
} // namespace UE
