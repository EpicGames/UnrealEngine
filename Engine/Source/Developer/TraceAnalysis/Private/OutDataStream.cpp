// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/OutDataStream.h"

#include "HAL/PlatformFileManager.h"

namespace UE::Trace {

//--------------------------------------------------------------------
// FFileOutDataStream
//--------------------------------------------------------------------

FFileOutDataStream::FFileOutDataStream()
	: Handle(nullptr)
{
}

FFileOutDataStream::~FFileOutDataStream()
{
}

bool FFileOutDataStream::Open(const TCHAR* Path)
{
	Handle.Reset(FPlatformFileManager::Get().GetPlatformFile().OpenWrite(Path));
	if (Handle == nullptr)
	{
		return false;
	}
	return true;
}

int32 FFileOutDataStream::Write(void* Data, uint32 Size)
{
	if (Handle == nullptr)
	{
		return -1;
	}

	if (!Handle->Write((uint8*)Data, Size))
	{
		Close();
		return -1;
	}

	return Size;
}

void FFileOutDataStream::Close()
{
	Handle.Reset();
}

} // namespace UE::Trace
