// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/SecureDataStream.h"
#include "Containers/AnsiString.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "DataStreamInternal.h"
#include "DataStreamInternal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Event.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/ScopeExit.h"
#include "Sockets.h"
#include "Templates/UniquePtr.h"

#if WITH_SHARED_ASIO_SSL_SUPPORT
#include "asio/ssl/stream.hpp"
#include "asio/ssl/error.hpp"
#include "asio/ssl/context.hpp"
#include <openssl/ssl.h>
#endif

#if PLATFORM_LINUX
#include <sys/socket.h>
#endif

// Enable extra debugging for secure connections
#define UE_TRACE_ANALYSIS_SSS_DEBUG_ENABLE 0

#if UE_TRACE_ANALYSIS_SSS_ENABLED 

namespace UE::Trace {

struct FSecureSocketContext
{
	FSecureSocketContext()
		: Context(1)
		, SSLContext(asio::ssl::context::sslv23_client)
	{
#if UE_TRACE_ANALYSIS_SSS_DEBUG_ENABLE // Enables additional SSL debug output
		SSL_CTX_set_info_callback(SSLContext.native_handle(), [](const SSL* Ssl, int Type, int Value) {
			{
				UE_LOGF(LogTraceData, VeryVerbose, 
					"TLS Info [%s]|Type [%s]|Desc %d/[%s]", 
					(Type & SSL_CB_READ) ? "Read" : "Write", 
					SSL_alert_type_string_long(Type), 
					Value,
					SSL_alert_desc_string_long(Value)
				);
			}
		});

		SSL_CTX_set_msg_callback(SSLContext.native_handle(), [](int write_p, int version, int content_type, const void *buf, size_t len, SSL *ssl, void *arg) {
			UE_LOGF(LogTraceData, VeryVerbose, 
				"TLS Traffic [%s]|Type [%d]|Length [%zu]", 
				write_p ? "In" : "Out", 
				content_type, 
				len
			);
		});
#endif
	}

	~FSecureSocketContext() 
	{
		Socket.Reset();	
	}

	bool OnVerifyCertificate(bool bPreverified, asio::ssl::verify_context &InContext)
	{
		char SubjectName[256];
		X509 *Certificate = X509_STORE_CTX_get_current_cert(InContext.native_handle());
		X509_NAME_oneline(X509_get_subject_name(Certificate), SubjectName, 256);
		UE_LOGF(LogTraceData, Verbose, "Verifying certificate '%s'", SubjectName)
		return bPreverified;
	}

	asio::io_context Context;
	asio::ssl::context SSLContext;
	TUniquePtr<asio::ssl::stream<asio::ip::tcp::socket>> Socket;
};

FSecureSocketStream::FSecureSocketStream()
	: ReadyToReadEvent(FPlatformProcess::GetSynchEventFromPool(false))
	, State(EState::None)
{
	AsioContext = MakeUnique<FSecureSocketContext>();
}

FSecureSocketStream::FSecureSocketStream(TUniquePtr<FArchiveFileWriterGeneric>&& InFileWriter)
	: ReadyToReadEvent(FPlatformProcess::GetSynchEventFromPool(false))
	, FileWriter(MoveTemp(InFileWriter))
	, State(EState::None)
{
	AsioContext = MakeUnique<FSecureSocketContext>();
}

FSecureSocketStream::~FSecureSocketStream()
{
	// Deregister our app_data value, if it was set with pre shared keys callback,
	// otherwise asio thinks it's a verify callback base and try to delete it.
	// Note that we check if the value is self, and only then reset it, otherwise we
	// will leak the internal asio object.
	SSL_CTX *NativeContext = AsioContext->SSLContext.native_handle();
	if (Mode == ESecureMode::PreSharedKeys && NativeContext && SSL_CTX_get_app_data(NativeContext) == this)
	{
		SSL_CTX_set_app_data(NativeContext, nullptr);
	}

	if (AsioContext)
	{
		AsioContext->Context.stop();
	}

	if (IoThread)
	{
		IoThread->WaitForCompletion();
		IoThread.Reset();
	}

	AsioContext.Reset();
}


bool FSecureSocketStream::LoadCertificateAuthority(FStringView InCertificatePath)
{
	SSL_CTX *NativeContextHandle = AsioContext->SSLContext.native_handle();
	if (!NativeContextHandle) {
		UE_LOGF(LogTraceData, Error, "Failed to obtain the the native SSL context handle");
		return false;
	}

	FAnsiString CertificatePath(InCertificatePath);
	if (!SSL_CTX_load_verify_locations(NativeContextHandle, *CertificatePath, nullptr))
	{
		UE_LOGF(LogTraceData, Error, "Failed to load certificate authority");
		return false;
	}

	return true;
}

bool FSecureSocketStream::SetCertificateAuthority(TConstArrayView<uint8> CA)
{
	const unsigned char* CAPointer = CA.GetData(); 
	size_t CASize = CA.Num();

	X509* DecodedCert = d2i_X509(NULL, &CAPointer, CASize);
	ON_SCOPE_EXIT { X509_free(DecodedCert); };

	if (!DecodedCert)
	{
		UE_LOGF(LogTraceData, Error, "Failed to decode SSL certificate authority (size: %d)", CA.Num());
		return false;
	}

	SSL_CTX *NativeContextHandle = AsioContext->SSLContext.native_handle();
	if (!NativeContextHandle) {
		UE_LOGF(LogTraceData, Error, "Failed to obtain the the native SSL context handle");
		return false;
	}

	X509_STORE *CertStore = SSL_CTX_get_cert_store(NativeContextHandle);
	if (!CertStore) {
		UE_LOGF(LogTraceData, Error, "Failed to obtain the cert store ");
		return false;
	}
	if (!X509_STORE_add_cert(CertStore, DecodedCert)) {
		UE_LOGF(LogTraceData, Error, "Failed to add certificate authority ");
		return false;
	}

	UE_LOGF(LogTraceData, Log, "Succesfully applied Certificate Authority to SSL context");
	return true;
}

struct FSecureStreamPSKCallback
{
	static unsigned int OnPSK(SSL* Ssl, const char* Hint, char* Identity, unsigned int MaxIdentityLength, unsigned char* PSK, unsigned int MaxPSKLength)
	{
		SSL_CTX* Context = SSL_get_SSL_CTX(Ssl);
		FSecureSocketStream* Self = (FSecureSocketStream*) SSL_CTX_get_app_data(Context);	
		if (!Self)
		{
			UE_LOGF(LogTraceData, Error, "Failed to retrieve app data");
			return 0;
		}

		FAnsiStringView PSKIdentity(Self->Identity);
		const int32 IdentityCount = PSKIdentity.CopyString(Identity, MaxIdentityLength);
		Identity[IdentityCount] = '\0';
		UE_LOGF(LogTraceData, Verbose, "Using identity '%s' (hint: '%s')", Identity, Hint);

		if (MaxPSKLength < uint32(Self->AuthToken.Num()))
		{
			UE_LOGF(LogTraceData, Error, "Target PSK buffer too small.");
			return 0;
		}

		const uint32 SizeToCopy = Self->AuthToken.Num();
		FMemory::Memcpy(PSK, Self->AuthToken.GetData(), SizeToCopy);

#if UE_TRACE_ANALYSIS_SSS_DEBUG_ENABLE
		auto ByteToHex = [](uint8 *Data, uint32 Size, char *OutBuffer)
		{
			for (uint32 i = 0; i < Size; ++i)
			{
				int written = snprintf(OutBuffer, 3, "%02X", static_cast<int>(Data[i]));
				if (written < 2 || written > 2)
				{
					*OutBuffer = '\0';
					break;
				}
				OutBuffer += 2;
			}
			*OutBuffer = '\0';
		};

		char Hex[256];
		ByteToHex(PSK, SizeToCopy, Hex);
		UE_LOGF(LogTraceData, Verbose, "PSK: '%s'", Hex);
#endif // UE_TRACE_ANALYSIS_SSS_DEBUG_ENABLE

		return SizeToCopy;
	}
};

bool FSecureSocketStream::SetPreSharedKey()
{
	AsioContext->SSLContext.clear_options(asio::ssl::context::default_workarounds);

	SSL_CTX *NativeContextHandle = AsioContext->SSLContext.native_handle();
	if (!NativeContextHandle) {
		UE_LOGF(LogTraceData, Error, "Failed to obtain the native SSL context handle");
		return false;
	}

	if (!SSL_CTX_set_app_data(NativeContextHandle, this))
	{
		UE_LOGF(LogTraceData, Error, "Failed to set app data");
		return false;
	}

	SSL_CTX_set_min_proto_version(NativeContextHandle, TLS1_2_VERSION);
	SSL_CTX_set_psk_client_callback(NativeContextHandle, &FSecureStreamPSKCallback::OnPSK);

	static const char* CipherList = "ECDHE-PSK-CHACHA20-POLY1305";
	if (!SSL_CTX_set_cipher_list(NativeContextHandle, CipherList))
	{
		UE_LOGF(LogTraceData, Error, "Failed to set cipher list");
		return false;
	}

	static const char* GroupLists = "P-384:P-256";
	if (!SSL_CTX_set1_groups_list(NativeContextHandle, GroupLists))
	{
		UE_LOGF(LogTraceData, Error, "Failed to set group list");
		return false;
	}

	UE_LOGF(LogTraceData, VeryVerbose, "Cipher list: '%s', groups list: '%s'", CipherList, GroupLists);

	return true;
}


void FSecureSocketStream::SetState(EState NewState, bool bTriggerReady)
{
	if (State != NewState)
	{
#if UE_TRACE_ANALYSIS_SSS_DEBUG_ENABLE 
		auto LexStateToString = [](EState State) -> const TCHAR* {
			switch(State)
			{
			case EState::None: return TEXT("None");
			case EState::Closed: return TEXT("Closed");
			case EState::Connecting: return TEXT("Connecting");
			case EState::Handshaking: return TEXT("Handshaking");
			case EState::Authenticating: return TEXT("Authenticating");
			case EState::Open: return TEXT("Open");
			case EState::Failed: return TEXT("Failed");
			default: return TEXT("Unknown");
			}
		};
		UE_LOGF(LogTraceData, Verbose, "Secure connection state: %ls -> %ls", LexStateToString(State), LexStateToString(NewState));
#endif //UE_DEBUG
		State = NewState;
	}
	if (bTriggerReady)
	{
		ReadyToReadEvent->Trigger();
	}
}

bool FSecureSocketStream::ConnectWithCertificate(FStringView InHost, uint16 InPort, TConstArrayView<uint8> InAuthToken, FStringView InCertificatePath)
{
	if (InCertificatePath.IsEmpty() || !LoadCertificateAuthority(InCertificatePath))
	{
		return false;
	}
	Mode = ESecureMode::Certificates;
	return ConnectInternal(InHost, InPort, InAuthToken);
}

bool FSecureSocketStream::ConnectWithSelfSigned(FStringView InHost, uint16 InPort, TConstArrayView<uint8> InAuthToken, TConstArrayView<uint8> InCertificate)
{
	if (!SetCertificateAuthority(InCertificate))
	{
		return false;
	}
	
	Mode = ESecureMode::Certificates;
	return ConnectInternal(InHost, InPort, InAuthToken);
}

bool FSecureSocketStream::ConnectWithPreSharedKey(FStringView InHost, uint16 InPort, TConstArrayView<uint8> InPreSharedKey, FAnsiStringView InIdentity)
{
	if (!SetPreSharedKey())
	{
		return false;
	}
	Identity = InIdentity;
	Mode = ESecureMode::PreSharedKeys;
	return ConnectInternal(InHost, InPort, InPreSharedKey);
}

bool FSecureSocketStream::ConnectInternal(FStringView InHost, uint16 InPort, TConstArrayView<uint8> InAuthToken)
{
	// Create the socket object
	AsioContext->Socket = MakeUnique<asio::ssl::stream<asio::ip::tcp::socket>>(AsioContext->Context, AsioContext->SSLContext);
	check(Mode != ESecureMode::None);
	if (Mode == ESecureMode::Certificates)
	{
		AsioContext->Socket->set_verify_mode(asio::ssl::verify_peer);
		AsioContext->Socket->set_verify_callback(
			std::bind(
				&FSecureSocketContext::OnVerifyCertificate, 
				AsioContext.Get(), 
				std::placeholders::_1, 
				std::placeholders::_2
			)
		);
	}
	else
	{
		AsioContext->Socket->set_verify_mode(asio::ssl::verify_none);
	}
	
	//Convert host to ansistring
	TAnsiStringBuilder<128> AnsiHost;
	AnsiHost << InHost;
	TAnsiStringBuilder<10> AnsiPort;
	AnsiPort << InPort;
	AuthToken = InAuthToken;

	asio::ip::tcp::resolver Resolver(AsioContext->Context);
	auto Endpoints = Resolver.resolve(AnsiHost.ToString(), AnsiPort.ToString());

	State = EState::Connecting;

	asio::async_connect(
		AsioContext->Socket->lowest_layer(), 
		Endpoints,
		[this](const std::error_code &Error, const asio::ip::tcp::endpoint& Endpoint) {
			if (Error)
			{
				UE_LOGF(LogTraceData, Error, 
					"Failed to connect to tracing host '%s' on port %u: %s", 
					Endpoint.address().to_string().c_str(),
					Endpoint.port(),
					Error.message().c_str()
				);
				SetState(EState::Failed, true);
				return;
			}
			UE_LOGF(LogTraceData, Display, 
				"Secure connection opened to '%s' on port %u.",
				Endpoint.address().to_string().c_str(),
				Endpoint.port()
			);
			AsyncHandshake();
		}
	);

	IoThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, TEXT("FSecureConnectionStream")));
	return true;
}

void FSecureSocketStream::AsyncHandshake()
{
	State = EState::Handshaking;

	AsioContext->Socket->async_handshake(
		asio::ssl::stream_base::client, 
		[this](const std::error_code &Error) 
		{
			if (Error)
			{
				UE_LOGF(LogTraceData, Error, 
					"Failed secure handshake: %s", 
					Error.message().c_str()
				);
				SetState(EState::Failed, true);
				return;
			}
			UE_LOGF(LogTraceData, Verbose, "Secure handshake succeeded");
			AsyncAuthenticate();
		}
	);
}

void FSecureSocketStream::AsyncAuthenticate(int32 Offset)
{
	State = EState::Authenticating;
	AsioContext->Socket->async_write_some(
		asio::buffer(AuthToken.GetData() + Offset, AuthToken.Num() - Offset),
		[this, Offset] (asio::error_code Error, size_t BytesWritten)
		{
			if (Error)
			{
				UE_LOGF(LogTraceData, Error, 
					"Failed to write authorization token: %s", 
					Error.message().c_str()
				);
				SetState(EState::Failed, true);
				return;
			}
			int32 CurrentOffset = Offset + BytesWritten;
			if (CurrentOffset != AuthToken.Num())
			{
				AsyncAuthenticate(CurrentOffset);
				return;
			}
			UE_LOGF(LogTraceData, Verbose, "Authentication sent");
			SetState(EState::Open, true);
			ReadyToReadEvent->Trigger();
#if UE_TRACE_ANALYSIS_SSS_BUFFERED
			AsyncRecieveData();
#endif
		}
	);
}

#if UE_TRACE_ANALYSIS_SSS_BUFFERED
void FSecureSocketStream::AsyncRecieveData()
{
	auto Handler = [this](const asio::error_code Error, size_t BytesRead)
	{
		if (Error == asio::error::eof || Error == asio::ssl::error::stream_truncated)
		{
			UE_LOGF(LogTraceData, Log, 
				"Connection closed by tracing peer"
			);
			State = EState::Closed;
			return;
		}
		else if (Error)
		{
			UE_LOGF(LogTraceData, Error, 
				"Read error recieving data: %s",
				Error.message().c_str()
			);
			State = EState::Failed;
			return;
		}
		
		if (BytesRead)
		{
			// Accumulate small reads into ReadBuffer
			BufferedBytes += BytesRead;

			// Flush buffered data to queue if the reader is waiting
			// or read buffer is full.
			if (ReadQueue.IsEmpty() || BufferedBytes == ReadBuffer.Num())
			{
				//UE_LOGF(LogTraceData, VeryVerbose, "[IO] Flushing %d bytes to queue", BufferedBytes);
				ReadQueue.Enqueue(ReadBuffer.GetData(), BufferedBytes);
				BufferedBytes = 0u;
			}
		}

		AsyncRecieveData();
	};

	if (!ReadBuffer.Num())
	{
		ReadBuffer.AddUninitialized(BufferCapacity);
	}
	uint8* ReadBufferPtr = ReadBuffer.GetData() + BufferedBytes;
	int32 BytesToRead = ReadBuffer.Num() - BufferedBytes; 
	checkf(BytesToRead > 0, TEXT("BytesToRead: %d, BufferSize: %d"), BytesToRead, ReadBuffer.Num());
	AsioContext->Socket->async_read_some(asio::buffer(ReadBufferPtr, BytesToRead), Handler);
}
#endif

uint32 FSecureSocketStream::Run()
{
	UE_LOGF(LogTraceData, Verbose, "Starting IO thread for secure connection stream");
	AsioContext->Context.run();
	UE_LOGF(LogTraceData, Verbose, "Finishing IO thread for secure connection stream");
	return 0;
}

void FSecureSocketStream::Stop()
{
	AsioContext->Context.stop();
}

bool FSecureSocketStream::WaitUntilReady()
{
	return ReadyToReadEvent->Wait(0, true);
}

int32 FSecureSocketStream::Read(void* OutData, uint32 Size)
{
	auto MaybeWriteFile = [this](void* Data, uint32 Size)
	{
		if (Size > 0 && FileWriter.IsValid() /*&& !FileWriter->IsError()*/)
		{
			FileWriter->Serialize(Data, Size);
		}
		return Size;
	};

	const int32 OutSize = int32(Size);
	int32 BytesRead = 0;
#if UE_TRACE_ANALYSIS_SSS_BUFFERED
	uint32 DataBlocksRead = 0;
	do {
		while (FData* NextData = ReadQueue.Peek())
		{
			// Can we fit next data block into the out buffer?
			if ((NextData->Num() + BytesRead) > OutSize)
			{
				return MaybeWriteFile(OutData, BytesRead);
			}
			// Peel off the block
			TOptional<FData> DequedData = ReadQueue.Dequeue();
			if (!DequedData)
			{
				return MaybeWriteFile(OutData, BytesRead);
			}
			FData& Data = DequedData.GetValue();
			FMemory::Memcpy((uint8*)OutData + BytesRead, Data.GetData(), Data.Num());
			BytesRead += Data.Num();
			if (++DataBlocksRead >= 10)
			{
				return MaybeWriteFile(OutData, BytesRead);
			}
		}
	}
	while (State == EState::Open && BytesRead < OutSize);
#else
	asio::error_code Error;
	BytesRead = AsioContext->Socket->read_some(asio::buffer(OutData, Size), Error);
	if (Error)
	{
		Close();
		if (Error == asio::error::eof || Error == asio::ssl::error::stream_truncated || Error == asio::error::connection_reset)
		{
			UE_LOGF(LogTraceData, Log, 
				"Connection closed by tracing peer"
			);
			return 0;
		}

		UE_LOGF(LogTraceData, Error, 
			"Read error recieving data (%d): %s",
			Error.value(),
			Error.message().c_str()
		);
		return Error.value() > 0 ? -Error.value() : Error.value();
	}
	check(BytesRead > 0); // If no bytes read an error should have been set.
#endif

	return MaybeWriteFile(OutData, BytesRead);
}

void FSecureSocketStream::Close()
{
	State = EState::Closed;
	FileWriter.Reset();
}

} // namespace UE::Trace

#endif // UE_TRACE_ANALYSIS_SSS_ENABLED 
