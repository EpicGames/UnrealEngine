// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Asio.h"
#include "CborReader.h"
#include "CborWriter.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/StringView.h"
#include "Serialization/MemoryArchive.h"
#include "Serialization/MemoryReader.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
template <int BufferSize>
struct TInlineMemoryWriter
	: public FMemoryArchive
{

	typedef TArray<uint8, TInlineAllocator<BufferSize>> BufferType;

					TInlineMemoryWriter();
	virtual void	Serialize(void* Data, int64 Num) override;
	BufferType		Buffer;
};

////////////////////////////////////////////////////////////////////////////////
template <int BufferSize>
inline TInlineMemoryWriter<BufferSize>::TInlineMemoryWriter()
{
	SetIsSaving(true);
}

////////////////////////////////////////////////////////////////////////////////
template <int BufferSize>
inline void TInlineMemoryWriter<BufferSize>::Serialize(void* Data, int64 Num)
{
	Buffer.Append((const uint8*)Data, IntCastChecked<int32>(Num));
}



////////////////////////////////////////////////////////////////////////////////
struct FPayload
{
	const uint8*	Data;
	uint32			Size;
};



////////////////////////////////////////////////////////////////////////////////
enum class EStatusCode
{
	Unknown				= 0,
	Success				= 200,
	BadRequest			= 400,
	NotFound			= 404,
	InternalError		= 500,
};



////////////////////////////////////////////////////////////////////////////////
template <int Size=128>
class TPayloadBuilder
{
public:
								TPayloadBuilder(EStatusCode StatusCode);
	template <int N>			TPayloadBuilder(const char (&Path)[N]);
	template <int N> void		AddInteger(const char (&Name)[N], int64 Value);
	template <int N> void		AddString(const char (&Name)[N], const char* Value, int32 Length=-1);
	template <int N> void		AddString(const char (&Name)[N], const FStringView& String);
	template <int N> void		AddStringArray(const char(&Name)[N], const TArray<FString>& Values);
	template <int N, typename T>
					void		AddArray(const char(&Name)[N], const TArrayView<T>& Values);
	FPayload					Done();

private:
	TInlineMemoryWriter<Size>	MemoryWriter;
	FCborWriter					CborWriter = { &MemoryWriter, ECborEndianness::StandardCompliant };
};

////////////////////////////////////////////////////////////////////////////////
template <int Size>
inline TPayloadBuilder<Size>::TPayloadBuilder(EStatusCode StatusCode)
{
	CborWriter.WriteContainerStart(ECborCode::Map, -1);
	AddInteger("$status", int32(StatusCode));
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
template <int N>
inline TPayloadBuilder<Size>::TPayloadBuilder(const char (&Path)[N])
{
	CborWriter.WriteContainerStart(ECborCode::Map, -1);
	AddString("$request", "GET");
	AddString("$path", Path, N - 1);
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
template <int N>
inline void TPayloadBuilder<Size>::AddInteger(const char (&Name)[N], int64 Value)
{
	CborWriter.WriteValue(Name, N - 1);
	CborWriter.WriteValue(Value);
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
template <int N>
inline void TPayloadBuilder<Size>::AddString(
	const char (&Name)[N],
	const char* Value,
	int Length)
{
	Length = (Length < 0) ? int32(strlen(Value)) : Length;
	CborWriter.WriteValue(Name, N - 1);
	CborWriter.WriteValue(Value, Length);
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
template <int N>
void TPayloadBuilder<Size>::AddString(const char(& Name)[N], const FStringView& String)
{
	
	CborWriter.WriteValue(Name, N - 1);
	CborWriter.WriteValue(String);
}

////////////////////////////////////////////////////////////////////////////////
template<int Size>
template<int N>
inline void TPayloadBuilder<Size>::AddStringArray(const char(&Name)[N], const TArray<FString>& Values)
{
	CborWriter.WriteValue(Name, N - 1);
	CborWriter.WriteContainerStart(ECborCode::Array, Values.Num());
	for (const FString& Elem : Values)
	{
		CborWriter.WriteValue(Elem);
	}
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
template <int N, typename T>
void TPayloadBuilder<Size>::AddArray(const char(& Name)[N], const TArrayView<T>& Values)
{
	CborWriter.WriteValue(Name, N - 1);
	CborWriter.WriteContainerStart(ECborCode::Array, Values.Num());
	for (const T& Elem : Values)
	{
		CborWriter.WriteValue(Elem);
	}
}

////////////////////////////////////////////////////////////////////////////////
template <int Size>
inline FPayload TPayloadBuilder<Size>::Done()
{
	CborWriter.WriteContainerEnd();
	return { MemoryWriter.Buffer.GetData(), uint32(MemoryWriter.Buffer.Num()) };
}



////////////////////////////////////////////////////////////////////////////////
class FResponse
{
public:
	EStatusCode		GetStatusCode() const;
	int64			GetInt64Checked(const char* Key, int64 Default) const;
	uint64			GetUint64Checked(const char* Key, uint64 Default) const;
	int32			GetInt32Checked(const char* Key, int32 Default) const;
	uint32			GetUint32Checked(const char* Key, uint32 Default) const;
	int64			GetInteger(const char* Key, int64 Default) const { return GetInt64Checked(Key, Default); }
	template <int N>
	FUtf8StringView	GetString(const char* Key, const char (&Default)[N]) const;
	bool			GetStringArray(const char* Key, TArray<FString>& OutArray) const;
	const uint8*	GetData() const;
	uint32			GetSize() const;
	uint8*			Reserve(uint32 Size);

private:
	template <typename Type, typename LambdaType>
	Type			GetValue(const char* Key, Type Default, LambdaType&& Lambda) const;
	TArray<uint8>	Buffer;
};

////////////////////////////////////////////////////////////////////////////////
inline EStatusCode FResponse::GetStatusCode() const
{
	int32 Code = GetInt32Checked("$status", 0);
	return Code ? EStatusCode(Code) : EStatusCode::Unknown;
}

////////////////////////////////////////////////////////////////////////////////
inline bool FResponse::GetStringArray(const char* Key, TArray<FString>& OutArray) const
{
	FMemoryReader MemoryReader(Buffer);
	FCborReader CborReader(&MemoryReader, ECborEndianness::StandardCompliant);
	FCborContext Context;
	while (CborReader.ReadNext(Context))
	{
		if (Context.IsString())
		{
			uint32 Length = uint32(Context.AsLength());
			uint32 Offset = uint32(MemoryReader.Tell());
			auto* String = (const char*)(Buffer.GetData() + Offset - Length);
			if (FCStringAnsi::Strncmp(Key, String, Length) == 0)
			{
				if (CborReader.ReadNext(Context) && Context.IsContainer() && Context.MajorType() == ECborCode::Array)
				{
					while (CborReader.ReadNext(Context) && Context.IsString())
					{
						OutArray.Emplace(Context.AsString());
					}
					return true;
				}
			}
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////
inline const uint8* FResponse::GetData() const
{
	return Buffer.GetData();
};

////////////////////////////////////////////////////////////////////////////////
inline uint32 FResponse::GetSize() const
{
	return Buffer.Num();
}

////////////////////////////////////////////////////////////////////////////////
inline uint8* FResponse::Reserve(uint32 Size)
{
	Buffer.SetNumUninitialized(Size, EAllowShrinking::No);
	return Buffer.GetData();
}

////////////////////////////////////////////////////////////////////////////////
template <typename Type, typename LambdaType>
inline Type FResponse::GetValue(const char* Key, Type Default, LambdaType&& Lambda) const
{
	FMemoryReader MemoryReader(Buffer);
	FCborReader CborReader(&MemoryReader, ECborEndianness::StandardCompliant);
	FCborContext Context;

	if (!CborReader.ReadNext(Context) || Context.MajorType() != ECborCode::Map)
	{
		return Default;
	}

	while (true)
	{
		// Read key
		if (!CborReader.ReadNext(Context) || !Context.IsString())
		{
			return Default;
		}

		uint32 Length = static_cast<uint32>(Context.AsLength());
		uint32 Offset = static_cast<uint32>(MemoryReader.Tell());
		auto* String = (const char*)(Buffer.GetData() + Offset - Length);
		bool bIsTarget = (FCStringAnsi::Strncmp(Key, String, Length) == 0);

		// Read value
		if (!CborReader.ReadNext(Context))
		{
			return Default;
		}

		if (bIsTarget)
		{
			return Lambda(Context, static_cast<uint32>(MemoryReader.Tell()));
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
inline int64 FResponse::GetInt64Checked(const char* Key, int64 Default) const
{
	return GetValue(
		Key,
		Default,
		[this] (const FCborContext& Context, int64)
		{
			return Context.AsInt();
		}
	);
}

////////////////////////////////////////////////////////////////////////////////
inline uint64 FResponse::GetUint64Checked(const char* Key, uint64 Default) const
{
	return GetValue(
		Key,
		Default,
		[this](const FCborContext& Context, uint64)
		{
			return Context.AsUInt();
		}
	);
}

////////////////////////////////////////////////////////////////////////////////
inline int32 FResponse::GetInt32Checked(const char* Key, int32 Default) const
{
	return GetValue(
		Key,
		Default,
		[this](const FCborContext& Context, int32)
		{
			int64 Value = Context.AsInt();
			check(Value >= INT_MIN && Value <= INT_MAX);
			return static_cast<int32>(Value);
		}
	);
}

////////////////////////////////////////////////////////////////////////////////
inline uint32 FResponse::GetUint32Checked(const char* Key, uint32 Default) const
{
	return GetValue(
		Key,
		Default,
		[this](const FCborContext& Context, uint32)
		{
			uint64 Value = Context.AsUInt();
			check(Value <= UINT_MAX);
			return static_cast<uint32>(Value);
		}
	);
}

////////////////////////////////////////////////////////////////////////////////
template <int N>
inline FUtf8StringView FResponse::GetString(const char* Key, const char (&Default)[N]) const
{
	FUtf8StringView DefaultView(Default, N - 1);
	return GetValue(
		Key,
		DefaultView,
		[this, DefaultView] (const FCborContext& Context, uint32 Offset)
		{
			if (Context.IsString())
			{
				uint32 Length = static_cast<uint32>(Context.AsLength());
				const char* Data = (const char*)(Buffer.GetData() + Offset - Length);
				return FUtf8StringView(Data, Length);
			}

			return DefaultView;
		}
	);
}

} // namespace Trace
} // namespace UE
