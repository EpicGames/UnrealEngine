// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/StoreClient.h"
#include "Algo/Transform.h"
#include "Asio.h"
#include "CborPayload.h"
#include "DataStreamInternal.h"
#include "Templates/UnrealTemplate.h"

namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
class FStoreCborClient
{
public:
							FStoreCborClient(asio::io_context& InIoContext);
							~FStoreCborClient();
	bool					IsOpen() const;
	void					Close();
	uint32					GetStoreAddress() const;
	uint32					GetStorePort() const;
	const FResponse&		GetResponse() const;
	bool					Connect(const TCHAR* Host, uint16 Port);
	bool					GetStatus() const;
	bool					GetVersion() const;
	bool					GetTraceCount() const;
	bool					GetTraceInfo(uint32 Index) const;
	bool					GetTraceInfoById(uint32 Id) const;
	FTraceDataStream*		ReadTrace(uint32 Id) const;
	bool					GetSessionCount() const;
	bool					GetSessionInfo(uint32 Index) const;
	bool					GetSessionInfoById(uint32 Id) const;
	bool					GetSessionInfoByTraceId(uint32 TraceId) const;
	bool					GetSessionInfoByTraceGuid(const FGuid& TraceGuid) const;
	bool					SetStoreDirectories(const TCHAR* StoreDir, const TArray<FString>& AddWatchDirs, const TArray<FString>& RemoveWatchDirs);
	bool					SetSponsored(bool bSponsored);

private:
	bool					Communicate(const FPayload& Payload) const;

	asio::io_context&				IoContext;
	mutable asio::ip::tcp::socket	Socket;
	mutable FResponse				Response;
	FString							Host;
	uint16							Port;
};

////////////////////////////////////////////////////////////////////////////////
FStoreCborClient::FStoreCborClient(asio::io_context& InIoContext)
: IoContext(InIoContext)
, Socket(InIoContext)
{
}

////////////////////////////////////////////////////////////////////////////////
FStoreCborClient::~FStoreCborClient()
{
	Close();
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::IsOpen() const
{
	return Socket.is_open();
}

////////////////////////////////////////////////////////////////////////////////
void FStoreCborClient::Close()
{
	Socket.close();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreCborClient::GetStoreAddress() const
{
	if (!IsOpen())
	{
		return 0;
	}

	const asio::ip::tcp::endpoint Endpoint = Socket.remote_endpoint();
	asio::ip::address Address = Endpoint.address();
	return Address.is_v4() ? Address.to_v4().to_uint() : 0;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreCborClient::GetStorePort() const
{
	if (!IsOpen())
	{
		return 0;
	}

	const asio::ip::tcp::endpoint Endpoint = Socket.remote_endpoint();
	return Endpoint.port();
}

////////////////////////////////////////////////////////////////////////////////
const FResponse& FStoreCborClient::GetResponse() const
{
	return Response;
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::Connect(const TCHAR* InHost, uint16 InPort)
{
	InPort = (InPort == 0) ? 1989 : InPort;

	FTCHARToUTF8 HostUtf8(InHost);
	char PortString[8];
	FCStringAnsi::Sprintf(PortString, "%d", InPort);

	asio::ip::tcp::resolver Resolver(IoContext);
	asio::ip::tcp::resolver::results_type Endpoints = Resolver.resolve(
		asio::ip::tcp::resolver::protocol_type::v4(),
		(const char*)HostUtf8.Get(),
		PortString);

	asio::error_code ErrorCode;
#if PLATFORM_WINDOWS
	DWORD Flags = WSA_FLAG_NO_HANDLE_INHERIT|WSA_FLAG_OVERLAPPED;
	SOCKET WinSocket = WSASocket(AF_INET, SOCK_STREAM, IPPROTO_TCP, nullptr, 0, Flags);
	Socket.assign(asio::ip::tcp::v4(), WinSocket);
#else
	Socket.open(asio::ip::tcp::v4());
#endif

	asio::connect(Socket, Endpoints, ErrorCode);
	if (ErrorCode)
	{
		return false;
	}

	// Save connection details if we need to reconnect
	Host = InHost;
	Port = InPort;

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::Communicate(const FPayload& Payload) const
{
	if (!Socket.is_open())
	{
		return false;
	}

	asio::error_code ErrorCode;

	// Send the payload
	uint32 PayloadSize = Payload.Size;
	asio::write(Socket, asio::buffer(&PayloadSize, sizeof(PayloadSize)), ErrorCode);
	asio::write(Socket, asio::buffer(Payload.Data, Payload.Size), ErrorCode);
	if (ErrorCode)
	{
		Socket.close();
		return false;
	}

	// Wait for a response
	uint32 ResponseSize = 0;
	asio::read(Socket, asio::buffer(&ResponseSize, sizeof(ResponseSize)), ErrorCode);
	if (ErrorCode)
	{
		Socket.close();
		return false;
	}

	if (ResponseSize == 0)
	{
		Socket.close();
		return false;
	}

	uint8* Dest = Response.Reserve(ResponseSize);
	asio::read(Socket, asio::buffer(Dest, ResponseSize), ErrorCode);
	if (ErrorCode)
	{
		Socket.close();
		return false;
	}

	if (Response.GetStatusCode() != EStatusCode::Success)
	{
		return false;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetStatus() const
{
	TPayloadBuilder<32> Builder("v1/status");
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetVersion() const
{
	TPayloadBuilder<32> Builder("v1/version");
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetTraceCount() const
{
	TPayloadBuilder<32> Builder("v1/trace/count");
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetTraceInfo(uint32 Index) const
{
	TPayloadBuilder<> Builder("v1/trace/info");
	Builder.AddInteger("index", Index);
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetTraceInfoById(uint32 Id) const
{
	TPayloadBuilder<> Builder("v1/trace/info");
	Builder.AddInteger("id", int32(Id));
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
FTraceDataStream* FStoreCborClient::ReadTrace(uint32 Id) const
{
	TPayloadBuilder<> Builder("v1/trace/read");
	Builder.AddInteger("id", Id);
	FPayload Payload = Builder.Done();
	if (!Communicate(Payload))
	{
		return nullptr;
	}

	uint32 SenderPort = Response.GetUint32Checked("port", 0);
	if (!SenderPort)
	{
		return nullptr;
	}

	asio::ip::address ServerAddr = Socket.remote_endpoint().address();
	asio::ip::tcp::endpoint Endpoint(ServerAddr, uint16(SenderPort));

	asio::error_code ErrorCode;
	asio::ip::tcp::socket SenderSocket(IoContext);
#if PLATFORM_WINDOWS
	DWORD Flags = WSA_FLAG_NO_HANDLE_INHERIT|WSA_FLAG_OVERLAPPED;
	SOCKET WinSocket = WSASocket(AF_INET, SOCK_STREAM, IPPROTO_TCP, nullptr, 0, Flags);
	SenderSocket.assign(asio::ip::tcp::v4(), WinSocket);
#else
	SenderSocket.open(asio::ip::tcp::v4());
#endif

	SenderSocket.connect(Endpoint, ErrorCode);
	if (ErrorCode)
	{
		return nullptr;
	}

	return new FTraceDataStream(SenderSocket);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetSessionCount() const
{
	TPayloadBuilder<> Builder("v1/session/count");
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetSessionInfo(uint32 Index) const
{
	TPayloadBuilder<> Builder("v1/session/info");
	Builder.AddInteger("index", int32(Index));
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetSessionInfoById(uint32 Id) const
{
	TPayloadBuilder<> Builder("v1/session/info");
	Builder.AddInteger("id", int32(Id));
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetSessionInfoByTraceId(uint32 TraceId) const
{
	TPayloadBuilder<> Builder("v1/session/info");
	Builder.AddInteger("trace_id", int32(TraceId));
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::GetSessionInfoByTraceGuid(const FGuid& TraceGuid) const
{
	TPayloadBuilder<> Builder("v1/session/info");
	Builder.AddString("trace_guid", TCHAR_TO_ANSI(*TraceGuid.ToString()));
	FPayload Payload = Builder.Done();
	return Communicate(Payload);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::SetStoreDirectories(const TCHAR* StoreDir, const TArray<FString>& AddWatchDirs, const TArray<FString>& RemoveWatchDir)
{
	TPayloadBuilder<> Builder("v1/settings/write");
	if (StoreDir)
	{
		Builder.AddString("StoreDir", FStringView(StoreDir));
	}
	TArray<FString> WatchDirs;
	if (!RemoveWatchDir.IsEmpty())
	{
		Algo::Transform(RemoveWatchDir, WatchDirs, [](const FString& In) { return TEXT("-") + In; });
	}
	if (!AddWatchDirs.IsEmpty())
	{
		WatchDirs.Append(AddWatchDirs);
	}
	if (!WatchDirs.IsEmpty())
	{
		Builder.AddStringArray("Additionalwatchdirs", WatchDirs);
	}
	return Communicate(Builder.Done());
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreCborClient::SetSponsored(bool bSponsored)
{
	TPayloadBuilder<> Builder("v1/settings/write");
	Builder.AddInteger("Sponsored", bSponsored ? 1 : 0);
	return Communicate(Builder.Done());
}



////////////////////////////////////////////////////////////////////////////////
// FStoreClient::FStatus
////////////////////////////////////////////////////////////////////////////////
FUtf8StringView FStoreClient::FStatus::GetStoreDir() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetString("store_dir", "");
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FStatus::GetRecorderPort() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("recorder_port", 0);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FStatus::GetStorePort() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("store_port", 0);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreClient::FStatus::GetSponsored() const
{
	const FResponse* Response = (const FResponse*)this;
	return !!Response->GetInteger("sponsored", 0);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FStatus::GetChangeSerial() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("change_serial", 0);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FStatus::GetSettingsSerial() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("settings_serial", 0);
}

////////////////////////////////////////////////////////////////////////////////
void FStoreClient::FStatus::GetWatchDirectories(TArray<FString>& OutDirs) const
{
	const FResponse* Response = (const FResponse*)this;
	Response->GetStringArray("watch_dirs", OutDirs);
}



////////////////////////////////////////////////////////////////////////////////
// FStoreClient::FVersion
////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FVersion::GetMajorVersion() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("major", 0);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FVersion::GetMinorVersion() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("minor", 0);
}

////////////////////////////////////////////////////////////////////////////////
FUtf8StringView FStoreClient::FVersion::GetConfiguration() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetString("configuration", "unknown");
}



////////////////////////////////////////////////////////////////////////////////
// FStoreClient::FTraceInfo
////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FTraceInfo::GetId() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("id", 0);
}

////////////////////////////////////////////////////////////////////////////////
uint64 FStoreClient::FTraceInfo::GetSize() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint64Checked("size", 0);
}

////////////////////////////////////////////////////////////////////////////////
uint64 FStoreClient::FTraceInfo::GetTimestamp() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint64Checked("timestamp", 0);
}

////////////////////////////////////////////////////////////////////////////////
FUtf8StringView FStoreClient::FTraceInfo::GetUri() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetString("uri", "");
}

////////////////////////////////////////////////////////////////////////////////
FUtf8StringView FStoreClient::FTraceInfo::GetName() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetString("name", "nameless");
}

////////////////////////////////////////////////////////////////////////////////
FUtf8StringView FStoreClient::FTraceInfo::GetMetadataUri() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetString("metadata_uri", "");
}



////////////////////////////////////////////////////////////////////////////////
// FStoreClient::FSessionInfo
////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FSessionInfo::GetId() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("id", 0);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FSessionInfo::GetTraceId() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("trace_id", 0);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FSessionInfo::GetIpAddress() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("ip_address", 0);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::FSessionInfo::GetControlPort() const
{
	const FResponse* Response = (const FResponse*)this;
	return Response->GetUint32Checked("control_port", 0);
}



////////////////////////////////////////////////////////////////////////////////
// FStoreClient
////////////////////////////////////////////////////////////////////////////////
FStoreClient* FStoreClient::Connect(const TCHAR* Host, uint32 Port)
{
	static asio::io_context IoContext;

	FStoreCborClient* Impl = new FStoreCborClient(IoContext);
	if (!Impl->Connect(Host, static_cast<uint16>(Port)))
	{
		delete Impl;
		return nullptr;
	}

	return (FStoreClient*)Impl;
}

////////////////////////////////////////////////////////////////////////////////
TUniquePtr<FStoreClient> FStoreClient::Create(const TCHAR* Host, uint32 Port)
{
	static asio::io_context IoContext;

	FStoreCborClient* Impl = new FStoreCborClient(IoContext);
	if (!Impl->Connect(Host, static_cast<uint16>(Port)))
	{
		delete Impl;
		return nullptr;
	}

	FStoreClient* Ptr = (FStoreClient*)Impl;
	return TUniquePtr<FStoreClient>(Ptr);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreClient::Reconnect(const TCHAR* Host, uint32 Port)
{
	FStoreCborClient* Self = (FStoreCborClient*)this;
	return Self->Connect(Host, static_cast<uint16>(Port));
}

////////////////////////////////////////////////////////////////////////////////
void FStoreClient::operator delete (void* Addr)
{
	FStoreCborClient* Self = (FStoreCborClient*)Addr;
	delete Self;
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreClient::IsValid() const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	return Self->IsOpen();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::GetStoreAddress() const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	return Self->GetStoreAddress();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::GetStorePort() const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	return Self->GetStorePort();
}

////////////////////////////////////////////////////////////////////////////////
const FStoreClient::FStatus* FStoreClient::GetStatus() const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetStatus())
	{
		return nullptr;
	}

	const FResponse& Response = Self->GetResponse();
	return (FStatus*)(&Response);
}

////////////////////////////////////////////////////////////////////////////////
const FStoreClient::FVersion* FStoreClient::GetVersion() const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetVersion())
	{
		return nullptr;
	}

	const FResponse& Response = Self->GetResponse();
	return (FVersion*)(&Response);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::GetTraceCount() const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetTraceCount())
	{
		return 0;
	}

	return Self->GetResponse().GetUint32Checked("count", 0);
}

////////////////////////////////////////////////////////////////////////////////
const FStoreClient::FTraceInfo* FStoreClient::GetTraceInfo(uint32 Index) const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetTraceInfo(Index))
	{
		return nullptr;
	}

	const FResponse& Response = Self->GetResponse();
	return (FTraceInfo*)(&Response);
}

////////////////////////////////////////////////////////////////////////////////
const FStoreClient::FTraceInfo* FStoreClient::GetTraceInfoById(uint32 Id) const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetTraceInfoById(Id))
	{
		return nullptr;
	}

	const FResponse& Response = Self->GetResponse();
	return (FTraceInfo*)(&Response);
}

////////////////////////////////////////////////////////////////////////////////
FStoreClient::FTraceData FStoreClient::ReadTrace(uint32 Id) const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	return FTraceData(Self->ReadTrace(Id));
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreClient::SetStoreDirectories(const TCHAR* StoreDir,
										const TArray<FString>& AddWatchDirs,
										const TArray<FString>& RemoveWatchDirs)
{
	FStoreCborClient* Self = (FStoreCborClient*)this;
	return Self->SetStoreDirectories(StoreDir, AddWatchDirs, RemoveWatchDirs);
}

////////////////////////////////////////////////////////////////////////////////
bool FStoreClient::SetSponsored(bool bSponsored)
{
	FStoreCborClient* Self = (FStoreCborClient*)this;
	return Self->SetSponsored(bSponsored);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStoreClient::GetSessionCount() const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetSessionCount())
	{
		return 0;
	}

	return Self->GetResponse().GetUint32Checked("count", 0);
}

////////////////////////////////////////////////////////////////////////////////
const FStoreClient::FSessionInfo* FStoreClient::GetSessionInfo(uint32 Index) const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetSessionInfo(Index))
	{
		return nullptr;
	}

	const FResponse& Response = Self->GetResponse();
	return (FSessionInfo*)(&Response);
}

////////////////////////////////////////////////////////////////////////////////
const FStoreClient::FSessionInfo* FStoreClient::GetSessionInfoById(uint32 Id) const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetSessionInfoById(Id))
	{
		return nullptr;
	}

	const FResponse& Response = Self->GetResponse();
	return (FSessionInfo*)(&Response);
}

////////////////////////////////////////////////////////////////////////////////
const FStoreClient::FSessionInfo* FStoreClient::GetSessionInfoByTraceId(uint32 TraceId) const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetSessionInfoByTraceId(TraceId))
	{
		return nullptr;
	}

	const FResponse& Response = Self->GetResponse();
	return (FSessionInfo*)(&Response);
}

////////////////////////////////////////////////////////////////////////////////
const FStoreClient::FSessionInfo* FStoreClient::GetSessionInfoByGuid(const FGuid& TraceGuid) const
{
	const FStoreCborClient* Self = (FStoreCborClient*)this;
	if (!Self->GetSessionInfoByTraceGuid(TraceGuid))
	{
		return nullptr;
	}

	const FResponse& Response = Self->GetResponse();
	return (FSessionInfo*)(&Response);
}

} // namespace Trace
} // namespace UE
