// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/StoreConnection.h"

#include "Misc/ScopeLock.h"

// TraceAnalysis
#include "Trace/StoreClient.h"

namespace UE::Trace
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FStoreConnection::FStoreConnection()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStoreConnection::~FStoreConnection()
{
	Disconnect();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FStoreConnection::ConnectToStore(const TCHAR* Host, uint32 Port)
{
	StoreClient = FStoreClient::Create(Host, Port);
	if (!StoreClient.IsValid())
	{
		return false;
	}

	LastStoreHost = Host;
	LastStorePort = Port;

	bIsLocalHost = LastStoreHost.Equals(TEXT("localhost"), ESearchCase::IgnoreCase) ||
				   LastStoreHost.Equals(TEXT("127.0.0.1"));

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FStoreConnection::ReconnectToStore()
{
	if (!StoreClient.IsValid())
	{
		return false;
	}

	return StoreClient->Reconnect(*LastStoreHost, LastStorePort);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStoreConnection::Disconnect()
{
	StoreClient.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FStoreConnection::GetStoreAddressAndPort(uint32& OutStoreAddress, uint32& OutStorePort) const
{
	if (!StoreClient.IsValid())
	{
		return false;
	}

	FScopeLock _(&StoreClientCriticalSection);
	OutStoreAddress = StoreClient->GetStoreAddress();
	OutStorePort = StoreClient->GetStorePort();
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString FStoreConnection::GetStoreDir() const
{
	if (!StoreClient.IsValid())
	{
		return FString();
	}

	FScopeLock _(&StoreClientCriticalSection);
	const UE::Trace::FStoreClient::FStatus* Status = StoreClient->GetStatus();
	return Status ? FString(Status->GetStoreDir()) : FString();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Trace
