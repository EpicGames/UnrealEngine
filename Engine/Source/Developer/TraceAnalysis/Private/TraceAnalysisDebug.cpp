// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceAnalysisDebug.h"

namespace UE::Trace::Analysis
{
	uint32 GDebugLevel = 0;
}
