// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceAnalysisModule.h"

#include "Modules/ModuleManager.h"

////////////////////////////////////////////////////////////////////////////////
void FTraceAnalysisModule::StartupModule()
{
}

////////////////////////////////////////////////////////////////////////////////
FName FTraceAnalysisModule::GetMessageLogName()
{
	static FName Name("TraceAnalysis");
	return Name;
}

////////////////////////////////////////////////////////////////////////////////
IMPLEMENT_MODULE(FTraceAnalysisModule, TraceAnalysis);
