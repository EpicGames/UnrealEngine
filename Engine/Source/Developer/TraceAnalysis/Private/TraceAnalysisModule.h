// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FTraceAnalysisModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;

	static FName GetMessageLogName();
};