// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/TraceWriter.h"

#include "Analysis/Transport/TidPacketTransport.h"
#include "HAL/PlatformTime.h"

// TraceLog
#include "Trace/Detail/Transport.h"

// TraceAnalysis
#include "Trace/OutDataStream.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Trace::Private
{

TRACELOG_API uint32 GetEncodeMaxSize(uint32 InputSize);
TRACELOG_API int32 Encode(const void* Src, int32 SrcSize, void* Dest, int32 DestSize);
TRACELOG_API int32 Decode(const void* Src, int32 SrcSize, void* Dest, int32 DestSize);

} // namespace UE::Trace::Private

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Trace::Private::WriterProtocol
{
	using Protocol7::EProtocol;
	using Protocol7::EFieldType;
	using Protocol7::FEventHeader;
	using Protocol7::FImportantEventHeader;
	using Protocol7::FEventHeaderSync;
	using Protocol7::FAuxHeader;
	using Protocol7::EEventFlags;
	using Protocol7::EFieldFamily;
	using Protocol7::FNewEventField;
	using Protocol7::FNewEventEvent;
	using Protocol7::EKnownEventUids;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Trace
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceWriterFieldTypeResolver
////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert(FTraceWriterFieldTypeResolver::GetType<uint64>() == ETraceWriterFieldType::Uint64, "");
static_assert(FTraceWriterFieldTypeResolver::GetType<const uint64>() == ETraceWriterFieldType::Uint64, "");
static_assert(FTraceWriterFieldTypeResolver::GetType<uint64[]>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Uint64), "");
static_assert(FTraceWriterFieldTypeResolver::GetType<const uint64[]>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Uint64), "");
static_assert(FTraceWriterFieldTypeResolver::GetType<TConstArrayView<uint64>>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Uint64), "");
static_assert(FTraceWriterFieldTypeResolver::GetType<TArrayView<uint64>>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Uint64), "");

static_assert(FTraceWriterFieldTypeResolver::GetType<void*>() == ETraceWriterFieldType::Pointer, "");
static_assert(FTraceWriterFieldTypeResolver::GetType<const void*>() == ETraceWriterFieldType::Pointer, "");
static_assert(FTraceWriterFieldTypeResolver::GetType<void* []>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Pointer), "");
static_assert(FTraceWriterFieldTypeResolver::GetType<const void* []>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Pointer), "");
static_assert(FTraceWriterFieldTypeResolver::GetType<TConstArrayView<void*>>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Pointer), "");
static_assert(FTraceWriterFieldTypeResolver::GetType<TArrayView<void*>>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Pointer), "");
static_assert(FTraceWriterFieldTypeResolver::GetType<TConstArrayView<const void*>>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Pointer), "");
static_assert(FTraceWriterFieldTypeResolver::GetType<TArrayView<const void*>>() == (ETraceWriterFieldType::ArrayFlag | ETraceWriterFieldType::Pointer), "");

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceWriterEventInfo
////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTraceWriterEventInfo::GetFieldIndex(FAnsiStringView InFieldName) const
{
	const uint32 FieldCount = Fields.Num();
	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		if (InFieldName.Equals(Fields[FieldIndex].GetName(), ESearchCase::CaseSensitive))
		{
			return FieldIndex;
		}
	}
	return FieldCount;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceWriterEventDeclarationBuilder
////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriterEventDeclarationBuilder::~FTraceWriterEventDeclarationBuilder()
{
	if (!bIsCompleted)
	{
		Writer.SetError(TEXT("An event declaration builder is destroyed without being completed. Check if an End() call is missing."));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriterEventDeclarationBuilder& FTraceWriterEventDeclarationBuilder::Field(FAnsiStringView InFieldName, ETraceWriterFieldType InFieldType)
{
	if (!EnumHasAnyFlags(InFieldType, ETraceWriterFieldType::ReferenceFlag | ETraceWriterFieldType::DefinitionIdFlag))
	{
		FTraceWriterEventFieldInfo& Field = EventInfo.Fields.Emplace_GetRef(InFieldName, InFieldType);
		Field.Size = FTraceWriter::GetByteSizeForFieldType(InFieldType);
		if (Field.Size == 0)
		{
			EventInfo.Flags |= ETraceWriterEventFlags::MaybeHasAux;
		}
	}
	else
	{
		Writer.SetError(TEXT("Invalid type for a regular event field!"));
	}
	return *this;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriterEventDeclarationBuilder& FTraceWriterEventDeclarationBuilder::ReferenceField(FAnsiStringView InFieldName, ETraceWriterFieldType InFieldType, uint16 InRefUid)
{
	if (InFieldType == ETraceWriterFieldType::Reference8 ||
		InFieldType == ETraceWriterFieldType::Reference16 ||
		InFieldType == ETraceWriterFieldType::Reference32 ||
		InFieldType == ETraceWriterFieldType::Reference64)
	{
		FTraceWriterEventFieldInfo& Field = EventInfo.Fields.Emplace_GetRef(InFieldName, InFieldType);
		Field.Size = FTraceWriter::GetByteSizeForFieldType(InFieldType);
		check(Field.Size > 0);
		Field.RefUid = InRefUid;
	}
	else
	{
		Writer.SetError(TEXT("Invalid type for a reference event field!"));
	}
	return *this;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriterEventDeclarationBuilder& FTraceWriterEventDeclarationBuilder::DefinitionIdField(ETraceWriterFieldType InFieldType)
{
	if (InFieldType == ETraceWriterFieldType::DefinitionId8 ||
		InFieldType == ETraceWriterFieldType::DefinitionId16 ||
		InFieldType == ETraceWriterFieldType::DefinitionId32 ||
		InFieldType == ETraceWriterFieldType::DefinitionId64)
	{
		FTraceWriterEventFieldInfo& Field = EventInfo.Fields.Emplace_GetRef(ANSITEXTVIEW("DefinitionId"), InFieldType);
		Field.Size = FTraceWriter::GetByteSizeForFieldType(InFieldType);
		check(Field.Size > 0);
	}
	else
	{
		Writer.SetError(TEXT("Invalid type for a definition event field!"));
	}
	return *this;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTraceWriterEventDeclarationBuilder::End()
{
	// Write the event declaration to the stream...
	Writer.WriteEventDeclaration(*this);

	const uint32 EventId = EventInfo.Id;

	// End() destroys the current EventDeclarationBuilder!
	bIsCompleted = true;
	delete this;

	return EventId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceWriterFieldData
////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriterFieldData::~FTraceWriterFieldData()
{
	if (Size > FTraceWriterFieldData::MaxInlineDataSize)
	{
		check(Buffer != nullptr);
		FMemory::Free(Buffer);
	}
	Data = 0;
	Data2 = 0;
	Size = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriterFieldData::SetData(const void* InData, uint32 InDataSize)
{
	if (InDataSize <= FTraceWriterFieldData::MaxInlineDataSize)
	{
		FMemory::Memcpy(&Data, InData, InDataSize);
	}
	else
	{
		check(Buffer == nullptr);
		Buffer = FMemory::Malloc(InDataSize, 0);
		FMemory::Memcpy(Buffer, InData, InDataSize);
	}
	Size = InDataSize;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceWriterEventBuilder
////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriterEventBuilder::~FTraceWriterEventBuilder()
{
	if (!bIsCompleted)
	{
		Writer.SetError(TEXT("An event builder is destroyed without being completed. Check if an End() call is missing."));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriterEventBuilder& FTraceWriterEventBuilder::Field(uint32 Index, const void* Data, uint32 DataSize)
{
	if (Writer.IsSafeModeEnabled())
	{
		if (Index >= EventInfo.GetNumFields())
		{
			SetErrorInvalidFieldIndex(Index);
			return *this;
		}
		const FTraceWriterEventFieldInfo& FieldInfo = EventInfo.GetField(Index);
		if (!EnumHasAnyFlags(FieldInfo.GetType(), ETraceWriterFieldType::ArrayFlag) ||
			EnumHasAnyFlags(FieldInfo.GetType(), ETraceWriterFieldType::ReferenceFlag | ETraceWriterFieldType::DefinitionIdFlag) ||
			FieldInfo.GetType() == (ETraceWriterFieldType::Bool | ETraceWriterFieldType::ArrayFlag) || // not supported by protocol 7
			FTraceWriter::GetByteSizeForFieldType(FieldInfo.GetType() & ~ETraceWriterFieldType::ArrayFlag) == 0)
		{
			SetErrorInvalidFieldType(Index);
			return *this;
		}
		if (DataSize >= (1u << 16) || FieldInfo.GetSize() != 0)
		{
			SetErrorInvalidFieldData(Index);
			return *this;
		}
	}
	Fields[Index].SetData(Data, DataSize);
	return *this;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceWriterEventBuilder::CheckValidPodField(uint32 Index, ETraceWriterFieldType FieldType, uint32 Size)
{
	if (Writer.IsSafeModeEnabled())
	{
		if (Index >= EventInfo.GetNumFields())
		{
			SetErrorInvalidFieldIndex(Index);
			return false;
		}
		const FTraceWriterEventFieldInfo& FieldInfo = EventInfo.GetField(Index);
		if (FieldInfo.GetSize() == 0 /*||
			FTraceWriter::IsCompatibleType(FieldInfo.GetType(), FieldType)*/)
		{
			SetErrorInvalidFieldType(Index);
			return false;
		}
		if (Size >= (1u << 16) || FieldInfo.GetSize() != Size)
		{
			SetErrorInvalidFieldData(Index);
			return false;
		}
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceWriterEventBuilder::CheckValidExactFieldType(uint32 Index, ETraceWriterFieldType FieldType)
{
	if (Writer.IsSafeModeEnabled())
	{
		if (Index >= EventInfo.GetNumFields())
		{
			SetErrorInvalidFieldIndex(Index);
			return false;
		}
		if (EventInfo.GetField(Index).GetType() != FieldType)
		{
			SetErrorInvalidFieldType(Index);
			return false;
		}
	}
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriterEventBuilder::SetField(uint32 Index, const void* Data, uint32 DataSize)
{
	check(Index < uint32(Fields.Num()));
	if (Writer.IsSafeModeEnabled())
	{
		const FTraceWriterEventFieldInfo& FieldInfo = EventInfo.GetField(Index);
		if (DataSize >= (1u << 16) ||
			(FieldInfo.GetSize() != 0 && FieldInfo.GetSize() != DataSize))
		{
			SetErrorInvalidFieldData(Index);
			return;
		}
	}
	Fields[Index].SetData(Data, DataSize);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriterEventBuilder::End()
{
	// Write the event to the stream...
	Writer.WriteEvent(*this);

	// End() destroys the current EventBuilder!
	bIsCompleted = true;
	delete this;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriterEventBuilder::SetErrorInvalidFieldIndex(uint32 Index)
{
	Writer.SetError(TEXT("Invalid field index!"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriterEventBuilder::SetErrorInvalidFieldType(uint32 Index)
{
	Writer.SetError(TEXT("Invalid field type!"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriterEventBuilder::SetErrorInvalidFieldData(uint32 Index)
{
	Writer.SetError(TEXT("Invalid field data!"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceWriter
////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriter::FTraceWriter(IOutDataStream& InStream)
	: Stream(InStream)
{
	GenerateGuid(&TraceGuid);
	GenerateGuid(&SessionGuid);

	EventInfos.SetNumZeroed(uint32(Private::WriterProtocol::EKnownEventUids::User), EAllowShrinking::No);

	InitDefaultClock();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriter::~FTraceWriter()
{
	Reset();

	if (EventBuffer)
	{
		FMemory::Free(EventBuffer);
		EventBuffer = nullptr;
	}
	EventBufferSize = 0;

	if (PacketBuffer)
	{
		FMemory::Free(PacketBuffer);
		PacketBuffer = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::Reset()
{
	Stream.Close();

	//TraceGuid
	//SessionGuid

	//////////////////////////////////////////////////

	CycleFrequency = 0;
	TimeGetter.Reset();
	StartCycle = 0;
	StartTimeSinceEpoch = 0.0;

	bUseCustomClock = false;
	InitDefaultClock();

	//////////////////////////////////////////////////

	NextSerial = 0;

	UnknownEventId = 0;
	NewTraceEventId = 0;
	ThreadTimingEventId = 0;
	ThreadInfoEventId = 0;
	ThreadGroupBeginEventId = 0;
	ThreadGroupEndEventId = 0;

	Threads.Reset();
	CurrentThreadId = ETransportTid::Events;

	for (FTraceWriterEventInfo* EventInfo : EventInfos)
	{
		if (EventInfo)
		{
			delete EventInfo;
		}
	}
	EventInfos.SetNum(uint32(Private::WriterProtocol::EKnownEventUids::User), EAllowShrinking::No);

	if (PendingEventDeclaration)
	{
		delete PendingEventDeclaration;
		PendingEventDeclaration = nullptr;
	}

	if (PendingEvent)
	{
		delete PendingEvent;
		PendingEvent = nullptr;
	}

	//EventBuffer
	//EventBufferSize
	EventBufferDataSize = 0;
	CompletedEventsDataSize = 0;

	//PacketBuffer

	LastError.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ETransport FTraceWriter::GetTransportProtocolVersion() const
{
	return ETransport::TidPacketSync;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint8 FTraceWriter::GetTraceProtocolVersion() const
{
	return (uint32)Private::WriterProtocol::EProtocol::Id;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::GenerateGuid(FTraceWriter::FTraceGuid* OutGuid)
{
	// This is not thread safe. Should only be accessed from the writer thread.
	// This initialized the prng with the current timestamp. In theory two machines could initialize on the exact same time
	// producing the same sequence of guids.
	static uint64 State = FPlatformTime::Cycles64();

	// L'Ecuyer, Pierre (1999). "Tables of Linear Congruential Generators of Different Sizes and Good Lattice Structure"
	// corrected with errata
	// Assuming m = 2e64
	constexpr uint64 C = 0x369DEA0F31A53F85;
	constexpr uint64 I = 1ull;

	const uint64 TopBits = State * C + I;
	const uint64 BottomBits = TopBits * C + I;
	State = BottomBits;

	*(uint64*)&OutGuid->Bits[0] = TopBits;
	*(uint64*)&OutGuid->Bits[2] = BottomBits;

	constexpr uint8 Version = 0x40; //Version 4, 4 bits
	constexpr uint8 VersionMask = 0xf0;
	constexpr uint8 Variant = 0x80; //Variant 1, 2 bits
	constexpr uint8 VariantMask = 0xc0;

	uint8* Octets = (uint8*)OutGuid;
	Octets[6] = Version | (~VersionMask & Octets[6]); // Octet 9
	Octets[8] = Variant | (~VariantMask & Octets[8]); // Octet 7
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::Begin(ETraceWriterBeginOptions Options)
{
	// Create the default threads.
	Threads.Reset();
	Threads.Add({ 0, FAnsiString("NewEvents") });
	Threads.Add({ 1, FAnsiString("Importants") });
	Threads.Add({ 2, FAnsiString("OnConnect") });
	static_assert(ETransportTid::Events == 0);
	static_assert(ETransportTid::Importants == 1);
	check(Threads.Num() == (uint32)FTidPacketTransport::EThreadStreamIndex::RealThreadsStart);

	constexpr uint16 GControlPort = 0;

	if (!bUseCustomClock)
	{
		InitDefaultClock();
	}

	// Handshake
	struct FHandshake
	{
		uint32 Magic = '2' | ('C' << 8) | ('R' << 16) | ('T' << 24);
		uint16 MetadataSize = uint16(MetadataSizeSum);

		uint16 MetadataField0 = uint16(sizeof(ControlPort) | (ControlPortFieldId << 8));
		uint16 ControlPort = uint16(GControlPort);

		uint16 MetadataField1 = uint16(sizeof(FTraceGuid) | (SessionGuidFieldId << 8));
		uint8 SessionGuid[16];	// Avoid padding

		uint16 MetadataField2 = uint16(sizeof(FTraceGuid) | (TraceGuidFieldId << 8));
		uint8 TraceGuid[16];	// Avoid padding

		enum
		{
			MetadataSizeSum = 2 + 2 + 2 + 16 + 2 + 16,
			Size = MetadataSizeSum + 4 + 2,
			ControlPortFieldId = 0,
			SessionGuidFieldId = 1,
			TraceGuidFieldId = 2,
		};
	};
	FHandshake Handshake;
	FMemory::Memcpy(&Handshake.SessionGuid, &SessionGuid, sizeof(FTraceGuid));
	FMemory::Memcpy(&Handshake.TraceGuid, &TraceGuid, sizeof(FTraceGuid));
	int32 Ret = Stream.Write(&Handshake, FHandshake::Size);
	if (Ret != FHandshake::Size)
	{
		SetError(TEXT("Failed to write trace header!"));
		return;
	}

	// Write versions.
	struct FVersions
	{
		uint8 TransportVersion = ETransport::TidPacketSync;
		uint8 ProtocolVersion = Private::WriterProtocol::EProtocol::Id;
	};
	FVersions Versions;
	Ret = Stream.Write(&Versions, sizeof(FVersions));
	if (Ret != sizeof(FVersions))
	{
		SetError(TEXT("Failed to write trace header!"));
		return;
	}

	if (EnumHasAnyFlags(Options, ETraceWriterBeginOptions::DeclareDefaultEvents))
	{
		// Describe initial events.
		DeclareDefaultEvents();
		WritePacket();
	}

	if (EnumHasAnyFlags(Options, ETraceWriterBeginOptions::WriteNewTraceEvent))
	{
		// Write $Trace.NewTrace event.
		WriteNewTraceEvent();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::End()
{
	WritePacket();
	Stream.Close();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::DeclareDefaultEvents()
{
	NewTraceEventId = DeclareEvent(ANSITEXTVIEW("$Trace"), ANSITEXTVIEW("NewTrace"), ETraceWriterEventFlags::ImportantNoSync)
		.Field(ANSITEXTVIEW("StartCycle"), ETraceWriterFieldType::Uint64)
		.Field(ANSITEXTVIEW("CycleFrequency"), ETraceWriterFieldType::Uint64)
		.Field(ANSITEXTVIEW("Endian"), ETraceWriterFieldType::Uint16)
		.Field(ANSITEXTVIEW("PointerSize"), ETraceWriterFieldType::Uint8)
		.Field(ANSITEXTVIEW("StartDateTime"), ETraceWriterFieldType::Float64)
		.End();

	ThreadTimingEventId = DeclareEvent(ANSITEXTVIEW("$Trace"), ANSITEXTVIEW("ThreadTiming"), ETraceWriterEventFlags::NoSync)
		.Field(ANSITEXTVIEW("BaseTimestamp"), ETraceWriterFieldType::Uint64)
		.End();

	ThreadInfoEventId = DeclareEvent(ANSITEXTVIEW("$Trace"), ANSITEXTVIEW("ThreadInfo"), ETraceWriterEventFlags::ImportantNoSync)
		.Field(ANSITEXTVIEW("ThreadId"), ETraceWriterFieldType::Uint32)
		.Field(ANSITEXTVIEW("SystemId"), ETraceWriterFieldType::Uint32)
		.Field(ANSITEXTVIEW("SortHint"), ETraceWriterFieldType::Int32)
		.Field(ANSITEXTVIEW("Name"), ETraceWriterFieldType::AnsiString)
		.End();

	ThreadGroupBeginEventId = DeclareEvent(ANSITEXTVIEW("$Trace"), ANSITEXTVIEW("ThreadGroupBegin"), ETraceWriterEventFlags::ImportantNoSync)
		.Field(ANSITEXTVIEW("Name"), ETraceWriterFieldType::AnsiString)
		.End();

	ThreadGroupEndEventId = DeclareEvent(ANSITEXTVIEW("$Trace"), ANSITEXTVIEW("ThreadGroupEnd"), ETraceWriterEventFlags::ImportantNoSync)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteNewTraceEvent()
{
	if (NewTraceEventId == 0)
	{
		SetError("$Trace.NewTrace event was not declared!");
		return;
	}

	SetCurrentThread(static_cast<uint32>(ETransportTid::Importants));

	WriteEvent(NewTraceEventId) // ImportantNoSync
		.Field(0, StartCycle)
		.Field(1, CycleFrequency)
		.Field(2, uint16(0x524d)) // Endian
		.Field(3, uint8(sizeof(void*))) // PointerSize
		.Field(4, StartTimeSinceEpoch)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTraceWriter::DeclareDiagnosticsSession2Event()
{
	return DeclareEvent(ANSITEXTVIEW("Diagnostics"), ANSITEXTVIEW("Session2"), ETraceWriterEventFlags::ImportantNoSync)
		.Field(ANSITEXTVIEW("ConfigurationType"), ETraceWriterFieldType::Uint8)
		.Field(ANSITEXTVIEW("TargetType"), ETraceWriterFieldType::Uint8)
		.Field(ANSITEXTVIEW("Changelist"), ETraceWriterFieldType::Uint32)
		.Field(ANSITEXTVIEW("InstanceId"), ETraceWriterFieldType::Uint32 | ETraceWriterFieldType::ArrayFlag)
		.Field(ANSITEXTVIEW("Platform"), ETraceWriterFieldType::AnsiString)
		.Field(ANSITEXTVIEW("AppName"), ETraceWriterFieldType::AnsiString)
		.Field(ANSITEXTVIEW("ProjectName"), ETraceWriterFieldType::WideString)
		.Field(ANSITEXTVIEW("Branch"), ETraceWriterFieldType::WideString)
		.Field(ANSITEXTVIEW("BuildVersion"), ETraceWriterFieldType::WideString)
		.Field(ANSITEXTVIEW("EngineVersion"), ETraceWriterFieldType::WideString)
		.Field(ANSITEXTVIEW("CommandLine"), ETraceWriterFieldType::WideString)
		.Field(ANSITEXTVIEW("VFSPaths"), ETraceWriterFieldType::AnsiString)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTraceWriter::GetOrDeclareUnknownEventId()
{
	if (UnknownEventId == 0)
	{
		UnknownEventId = DeclareEvent(ANSITEXTVIEW("TraceWriter"), ANSITEXTVIEW("UnknownEvent"), ETraceWriterEventFlags::ImportantNoSync)
			.End();
	}
	return UnknownEventId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FTraceWriter::RegisterThread(FAnsiStringView ThreadName, uint32 SystemId, int32 SortHint, bool bShouldWriteThreadInfo)
{
	const FTraceWriterThreadInfo* FoundThread = Threads.FindByPredicate(
		[&ThreadName](const FTraceWriterThreadInfo& ThreadInfo)
		{
			return ThreadInfo.Id != FTraceWriterThreadInfo::InvalidThreadId &&
				ThreadName.Equals(ThreadInfo.Name, ESearchCase::CaseSensitive);
		});
	if (FoundThread)
	{
		return FoundThread->Id;
	}

	const uint32 ThreadId = uint32(Threads.Num());
	RegisterCustomThread(ThreadId, ThreadName, SystemId, SortHint, bShouldWriteThreadInfo);
	return ThreadId;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::RegisterCustomThread(uint32 ThreadId, FAnsiStringView ThreadName, uint32 SystemId, int32 SortHint, bool bShouldWriteThreadInfo)
{
	const uint32 NumThreads = uint32(Threads.Num());

	if (ThreadId < NumThreads && Threads[ThreadId].Id != FTraceWriterThreadInfo::InvalidThreadId)
	{
		SetError("Thread already registered!");
		return;
	}

	if (ThreadId >= NumThreads)
	{
		Threads.AddDefaulted(ThreadId - NumThreads + 1);
	}

	FTraceWriterThreadInfo& Thread = Threads[ThreadId];
	Thread.Id = ThreadId;
	Thread.Name = FAnsiString(ThreadName);
	Thread.SystemId = SystemId;
	Thread.SortHint = SortHint;

	if (bShouldWriteThreadInfo)
	{
		WriteThreadInfo(Thread.Id, Thread.Name, Thread.SystemId, Thread.SortHint);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteThreadInfo(uint32 ThreadId)
{
	if (ThreadId >= uint32(Threads.Num()))
	{
		SetError("Thread is not registered!");
		return;
	}

	const FTraceWriterThreadInfo& Thread = Threads[ThreadId];
	WriteThreadInfo(Thread.Id, Thread.Name, Thread.SystemId, Thread.SortHint);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteThreadInfo(uint32 ThreadId, FAnsiStringView ThreadName, uint32 SystemId, int32 SortHint)
{
	if (ThreadInfoEventId == 0)
	{
		SetError("$Trace.ThreadInfo event was not declared!");
		return;
	}

	SetCurrentThreadImportants();

	WriteEvent(ThreadInfoEventId) // ImportantNoSync
		.Field(0, ThreadId)
		.Field(1, SystemId)
		.Field(2, SortHint)
		.Field(3, ThreadName)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::SetCurrentThread(uint32 ThreadId)
{
	if (ThreadId != CurrentThreadId)
	{
		// A packet can only contain events from a single thread.
		WritePacket();

		if ((EventBufferDataSize > 0) ||
			(PendingEvent != nullptr) ||
			(PendingEventDeclaration != nullptr))
		{
			SetError(TEXT("Cannot change the current thread while writing an event or an event declaration! Check if an End() call is missing."));
			return;
		}

		if (ThreadId >= uint32(Threads.Num()) ||
			Threads[ThreadId].Id == FTraceWriterThreadInfo::InvalidThreadId)
		{
			SetError(TEXT("Thread is not registered!"));
			return;
		}

		CurrentThreadId = ThreadId;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteThreadTimingEvent(uint64 BaseTimestamp)
{
	if (ThreadTimingEventId == 0)
	{
		SetError("$Trace.ThreadTiming event was not declared!");
		return;
	}

	WriteEvent(ThreadTimingEventId) // NoSync
		.Field(0, BaseTimestamp)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteThreadGroupBeginEvent(FAnsiStringView GroupName)
{
	if (ThreadGroupBeginEventId == 0)
	{
		SetError("$Trace.ThreadGroupBegin event was not declared!");
		return;
	}

	SetCurrentThreadImportants();

	WriteEvent(ThreadGroupBeginEventId) // ImportantNoSync
		.Field(0, GroupName)
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteThreadGroupEndEvent()
{
	if (ThreadGroupEndEventId == 0)
	{
		SetError("$Trace.ThreadGroupEnd event was not declared!");
		return;
	}

	SetCurrentThreadImportants();

	WriteEvent(ThreadGroupEndEventId) // ImportantNoSync
		.End();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FTraceWriterEventInfo* FTraceWriter::FindEvent(FAnsiStringView LoggerName, FAnsiStringView EventName) const
{
	for (const FTraceWriterEventInfo* EventInfo : EventInfos)
	{
		if (EventInfo &&
			LoggerName.Equals(EventInfo->LoggerName, ESearchCase::CaseSensitive) &&
			EventName.Equals(EventInfo->Name, ESearchCase::CaseSensitive))
		{
			return EventInfo;
		}
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriterEventDeclarationBuilder& FTraceWriter::DeclareEvent(FAnsiStringView LoggerName, FAnsiStringView Name, ETraceWriterEventFlags Flags)
{
	check(PendingEventDeclaration == nullptr);

	if (LoggerName.IsEmpty())
	{
		SetError(TEXT("Logger name cannot be empty!"));
		LoggerName = ANSITEXTVIEW("Unknown");
	}

	if (Name.IsEmpty())
	{
		SetError(TEXT("Event name cannot be empty!"));
		Name = ANSITEXTVIEW("Unknown");
	}

	if (EnumHasAnyFlags(Flags, ETraceWriterEventFlags::Important) &&
		!EnumHasAnyFlags(Flags, ETraceWriterEventFlags::NoSync))
	{
		SetError(TEXT("Important events cannot be sync!"));
		Flags = ETraceWriterEventFlags::ImportantNoSync;
	}

	FTraceWriterEventInfo* EventInfo = new FTraceWriterEventInfo(LoggerName, Name, Flags);
	EventInfo->Id = static_cast<uint32>(EventInfos.Num());
	EventInfos.Add(EventInfo);

	FTraceWriterEventDeclarationBuilder* Builder = new FTraceWriterEventDeclarationBuilder(*this, *EventInfo);
	PendingEventDeclaration = Builder;
	return *Builder;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteEventDeclaration(const FTraceWriterEventDeclarationBuilder& Builder)
{
	check(PendingEventDeclaration == &Builder);
	PendingEventDeclaration = nullptr;

	const FTraceWriterEventInfo& EventInfo = Builder.EventInfo;
	check(EventInfos.Contains(&EventInfo));

	SetCurrentThread(static_cast<uint32>(ETransportTid::Events));
	BeginEvent();
	WriteEventDeclaration(EventInfo);
	EndEvent();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteEventDeclaration(const FTraceWriterEventInfo& EventInfo)
{
	// See the equivalent implementation in the TraceLog module:
	//    TraceLog\Private\Trace\EventNode.cpp
	//    FEventNode::Describe()

	using namespace Private::WriterProtocol;

	//////////////////////////////////////////////////

	const uint32 FieldCount = EventInfo.GetNumFields();
	if (FieldCount > 255)
	{
		SetError(TEXT("Too many fields!"));
		return;
	}

	if (EventInfo.GetId() > uint32(EKnownEventUids::Max))
	{
		SetError(TEXT("Event id is too large!"));
		return;
	}

	//////////////////////////////////////////////////
	// Calculate the total size of names (strings).

	uint32 NamesSize = 0;

	if (EventInfo.GetLoggerName().Len() > 255)
	{
		SetError(TEXT("Logger name is too long!"));
		return;
	}
	NamesSize += EventInfo.GetLoggerName().Len();

	if (EventInfo.GetName().Len() > 255)
	{
		SetError(TEXT("Event name is too long!"));
		return;
	}
	NamesSize += EventInfo.GetName().Len();

	bool bMaybeHasAux = false;
	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		const FTraceWriterEventFieldInfo& FieldInfo = EventInfo.GetField(FieldIndex);
		if (!EnumHasAnyFlags(FieldInfo.Type, ETraceWriterFieldType::DefinitionIdFlag))
		{
			if (FieldInfo.GetName().Len() > 255)
			{
				SetError(TEXT("Field name is too long!"));
				return;
			}
			NamesSize += FieldInfo.GetName().Len();
		}
	}

	//////////////////////////////////////////////////
	// Compute the total event size.

	uint32 EventSize =
		  sizeof(FNewEventEvent)              // "NewEvent" event header
		+ FieldCount * sizeof(FNewEventField) // fields
		+ NamesSize;                          // strings
	EventSize = (EventSize + 1) & ~1;         // align to 2 to match TraceLog

	if (EventSize >= (1u << 16))
	{
		SetError(TEXT("NewEvent size is too large!"));
		return;
	}

	const uint32 TotalEventSize =
		  sizeof(FEventHeader) // event header (NoSync)
		+ sizeof(uint16)       // event size
		+ EventSize;           // event data

	//////////////////////////////////////////////////
	// Reserve buffer.

	uint8* Buffer = GetBuffer(TotalEventSize);
	if (!Buffer)
	{
		SetError(TEXT("Failed to write new event!"));
		return;
	}

	//////////////////////////////////////////////////
	// Write event header(s).

	FEventHeader* EventHeader = (FEventHeader*)Buffer;
	EventHeader->Uid = EKnownEventUids::NewEvent;
	Buffer += sizeof(FEventHeader);

	*(uint16*)Buffer = static_cast<uint16>(EventSize);
	Buffer += 2;

	FNewEventEvent& NewEvent = *(FNewEventEvent*)Buffer;

	NewEvent.EventUid = static_cast<uint16>(EventInfo.GetId());
	NewEvent.FieldCount = static_cast<uint8>(EventInfo.GetNumFields());
	NewEvent.Flags = GetProtocolEventFlags(EventInfo.GetFlags());
	NewEvent.LoggerNameSize = static_cast<uint8>(EventInfo.GetLoggerName().Len());
	NewEvent.EventNameSize = static_cast<uint8>(EventInfo.GetName().Len());

	// Write fields.
	uint32 Offset = 0;
	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		const FTraceWriterEventFieldInfo& FieldInfo = EventInfo.GetField(FieldIndex);
		FNewEventField& Field = *reinterpret_cast<FNewEventField*>(&NewEvent.Fields[FieldIndex]);
		Field.FieldType = GetProtocolFieldFamily(FieldInfo.GetType());
		Field.Unused = 0;
		const uint8 TypeInfo = GetProtocolFieldType(FieldInfo.GetType());
		switch (Field.FieldType)
		{
		case EFieldFamily::Regular:
			Field.Regular.Offset = static_cast<uint16>(Offset);
			Field.Regular.Size = FieldInfo.GetSize();
			Field.Regular.TypeInfo = TypeInfo;
			Field.Regular.NameSize = static_cast<uint8>(FieldInfo.GetName().Len());
			break;
		case EFieldFamily::Reference:
			Field.Reference.Offset = static_cast<uint16>(Offset);
			Field.Reference.RefUid = FieldInfo.GetRefUid();
			Field.Reference.TypeInfo = TypeInfo;
			Field.Reference.NameSize = static_cast<uint8>(FieldInfo.GetName().Len());
			break;
		case EFieldFamily::DefinitionId:
			Field.DefinitionId.Offset = static_cast<uint16>(Offset);
			Field.DefinitionId.Unused1 = 0;
			Field.DefinitionId.Unused2 = 0;
			Field.DefinitionId.TypeInfo = TypeInfo;
			break;
		default:
			SetError(TEXT("Invalid field type!"));
			return;
		}
		Offset += FieldInfo.GetSize();
	}

	// Write names.
	uint8* NamesCursor = (uint8*)(NewEvent.Fields + NewEvent.FieldCount);
	auto WriteName = [&NamesCursor](const FAnsiString& String)
		{
			FMemory::Memcpy(NamesCursor, *String, String.Len());
			NamesCursor += String.Len();
		};
	WriteName(EventInfo.GetLoggerName());
	WriteName(EventInfo.GetName());
	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		const FTraceWriterEventFieldInfo& FieldInfo = EventInfo.GetField(FieldIndex);
		if (!EnumHasAnyFlags(FieldInfo.Type, ETraceWriterFieldType::DefinitionIdFlag))
		{
			WriteName(FieldInfo.GetName());
		}
	}

	//////////////////////////////////////////////////
	// Commit the written buffer.

	AdvanceBuffer(TotalEventSize);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceWriterEventBuilder& FTraceWriter::WriteEvent(uint32 EventId)
{
	check(PendingEvent == nullptr);

	if (EventId >= static_cast<uint32>(EventInfos.Num()))
	{
		SetError(TEXT("Invalid event id!"));
		EventId = GetOrDeclareUnknownEventId();
	}

	check(EventId < static_cast<uint32>(EventInfos.Num()));
	check(EventInfos[EventId] != nullptr);

	FTraceWriterEventBuilder* Builder = new FTraceWriterEventBuilder(*this, *EventInfos[EventId]);
	PendingEvent = Builder;
	return *Builder;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteEvent(const FTraceWriterEventBuilder& Builder)
{
	using namespace Private::WriterProtocol;

	check(PendingEvent == &Builder);
	PendingEvent = nullptr;

	const FTraceWriterEventInfo& EventInfo = Builder.EventInfo;
	check(EventInfos.Contains(&EventInfo));

	const bool bMaybeHasAux = EnumHasAnyFlags(EventInfo.GetFlags(), ETraceWriterEventFlags::MaybeHasAux);
	const bool bIsImportant = EnumHasAnyFlags(EventInfo.GetFlags(), ETraceWriterEventFlags::Important);
	bool bIsNoSync = EnumHasAnyFlags(EventInfo.GetFlags(), ETraceWriterEventFlags::NoSync);

	if (bIsImportant)
	{
		if (!bIsNoSync)
		{
			SetError(TEXT("Important events must be NoSync!"));
			bIsNoSync = true;
		}
		if (CurrentThreadId != uint32(ETransportTid::Importants))
		{
			SetError(TEXT("Important events can only be emitted on the Importants thread!"));
			SetCurrentThread(static_cast<uint32>(ETransportTid::Importants));
		}
	}
	else
	{
		if (CurrentThreadId == uint32(ETransportTid::Importants))
		{
			SetError(TEXT("Non-important events can not be emitted on the Importants thread!"));
			SetCurrentThread(uint32(FTidPacketTransport::EThreadStreamIndex::RealThreadsStart));
		}
	}

	BeginEvent();
	WriteEvent(Builder, EventInfo);
	EndEvent();

	if (bMaybeHasAux)
	{
		// Write the AuxData events.
		const uint32 FieldCount = EventInfo.GetNumFields();
		for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
		{
			const FTraceWriterEventFieldInfo& FieldInfo = EventInfo.GetField(FieldIndex);
			uint32 FieldSize = FieldInfo.GetSize();
			if (FieldSize == 0)
			{
				const FTraceWriterFieldData& FieldData = Builder.Fields[FieldIndex];
				if (FieldData.GetSize() > 0)
				{
					WriteAuxData(FieldIndex, FieldData.GetData(), FieldData.GetSize(), bIsImportant);
				}
			}
		}

		// Write the AuxDataTerminal event.
		BeginEvent();
		uint8* Buffer = GetBuffer(1);
		if (Buffer)
		{
			*Buffer = bIsImportant ?
				uint8(EKnownEventUids::AuxDataTerminal) : // no _UidShift !!!
				uint8(EKnownEventUids::AuxDataTerminal << EKnownEventUids::_UidShift);
			AdvanceBuffer(1);
		}
		else
		{
			SetError(TEXT("Failed to write aux terminal event!"));
		}
		EndEvent();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteEvent(const FTraceWriterEventBuilder& Builder, const FTraceWriterEventInfo& EventInfo)
{
	// See the equivalent implementation in the TraceLog module:
	//    TraceLog\Private\Trace\*.cpp

	using namespace Private::WriterProtocol;

	//////////////////////////////////////////////////

	const uint32 FieldCount = EventInfo.GetNumFields();
	check(FieldCount <= 255);
	check(FieldCount == Builder.Fields.Num());

	//////////////////////////////////////////////////
	// Compute event size.

	uint32 EventSize = 0;
	uint32 AuxSize = 0;
	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		const FTraceWriterEventFieldInfo& FieldInfo = EventInfo.GetField(FieldIndex);
		EventSize += FieldInfo.GetSize();
		if (FieldInfo.GetSize() == 0)
		{
			const FTraceWriterFieldData& FieldData = Builder.Fields[FieldIndex];
			if (FieldData.GetSize() > 0)
			{
				AuxSize += sizeof(FAuxHeader) + FieldData.GetSize();
			}
		}
	}
	if (EnumHasAllFlags(EventInfo.GetFlags(), ETraceWriterEventFlags::MaybeHasAux))
	{
		AuxSize += 1; // aux data terminal
	}

	if (EventSize >= (1u << 16))
	{
		SetError(TEXT("Event size is too large!"));
		return;
	}

	uint32 TotalEventSize = 0;
	if (EnumHasAnyFlags(EventInfo.GetFlags(), ETraceWriterEventFlags::Important))
	{
		TotalEventSize = sizeof(FImportantEventHeader); // event header
	}
	else if (EnumHasAnyFlags(EventInfo.GetFlags(), ETraceWriterEventFlags::NoSync))
	{
		TotalEventSize = sizeof(FEventHeader); // event header
	}
	else // Sync
	{
		TotalEventSize = sizeof(FEventHeaderSync); // event header
	}
	TotalEventSize += EventSize; // event data

	//////////////////////////////////////////////////
	// Reserve buffer.

	uint8* Buffer = GetBuffer(TotalEventSize + AuxSize);
	if (!Buffer)
	{
		SetError(TEXT("Failed to write event!"));
		return;
	}

	//////////////////////////////////////////////////

	if (EnumHasAnyFlags(EventInfo.GetFlags(), ETraceWriterEventFlags::Important))
	{
		if (EventSize + AuxSize >= (1u << 16))
		{
			SetError(TEXT("Important event size is too large!"));
			return;
		}

		check(CurrentThreadId == uint32(ETransportTid::Importants));
		uint16 Uid16 = uint16(EventInfo.GetId()); // no Flag_TwoByteUid bit !!!
		FImportantEventHeader* EventHeader = (FImportantEventHeader*)Buffer;
		EventHeader->Uid = uint16(EventInfo.GetId());
		EventHeader->Size = uint16(EventSize + AuxSize); // includes size of aux data !!!
		Buffer += sizeof(FImportantEventHeader);
	}
	else if (EnumHasAnyFlags(EventInfo.GetFlags(), ETraceWriterEventFlags::NoSync))
	{
		FEventHeader* EventHeader = (FEventHeader*)Buffer;
		EventHeader->Uid = uint16(EventInfo.GetId() << EKnownEventUids::_UidShift) | EKnownEventUids::Flag_TwoByteUid;
		Buffer += sizeof(FEventHeader);
	}
	else // Sync
	{
		FEventHeaderSync* EventHeader = (FEventHeaderSync*)Buffer;
		FMemory::Memcpy(((uint8*)EventHeader->Data) - 3, &NextSerial, sizeof(NextSerial)); // FEventHeaderSync::SerialHigh,SerialLow
		++NextSerial;
		EventHeader->Uid = uint16(EventInfo.GetId() << EKnownEventUids::_UidShift) | EKnownEventUids::Flag_TwoByteUid;
		Buffer += sizeof(FEventHeaderSync);
	}

	for (uint32 FieldIndex = 0; FieldIndex < FieldCount; ++FieldIndex)
	{
		const FTraceWriterEventFieldInfo& FieldInfo = EventInfo.GetField(FieldIndex);
		uint32 FieldSize = FieldInfo.GetSize();
		if (FieldSize != 0)
		{
			const FTraceWriterFieldData& FieldData = Builder.Fields[FieldIndex];
			if (FieldData.GetSize() == FieldSize)
			{
				FMemory::Memcpy(Buffer, FieldData.GetData(), FieldData.GetSize());
			}
			else
			{
				if (FieldData.GetSize() != 0)
				{
					SetError(TEXT("Event field data does not match the field declaration!"));
				}
				FMemory::Memzero(Buffer, FieldSize);
			}
			Buffer += FieldSize;
		}
	}

	//////////////////////////////////////////////////
	// Commit the written buffer.

	AdvanceBuffer(TotalEventSize);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteAuxData(uint32 Index, const void* Data, uint32 Size, bool bIsImportant)
{
	// Technically, this could be FAuxHeader::SizeLimit (512 KiB), but in that case, it will exceed
	// the maximum size for a packet. The current packet size (even for uncompressed packets) is limited
	// to uint16 (see FTidPacketEncoded and FTidPacket).
	constexpr uint32 MaxAuxEventSize = 1 << 15;

	while (Size > 0)
	{
		const uint32 SegmentSize = FMath::Min(MaxAuxEventSize, Size);

		BeginEvent();
		WriteAuxDataSegment(Index, Data, SegmentSize, bIsImportant);
		EndEvent();

		Data = ((const uint8*)Data) + SegmentSize;
		Size -= SegmentSize;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteAuxDataSegment(uint32 Index, const void* SegmentData, uint32 SegmentSize, bool bIsImportant)
{
	using namespace Private::WriterProtocol;

	uint32 Size = sizeof(FAuxHeader) + SegmentSize;

	uint8* Buffer = GetBuffer(Size + 1);
	if (!Buffer)
	{
		SetError(TEXT("Failed to write aux event!"));
		return;
	}

	// Write header.
	uint32 Pack = SegmentSize << FAuxHeader::SizeShift;
	Pack |= Index << FAuxHeader::FieldShift;
	FMemory::Memcpy(Buffer, &Pack, sizeof(uint32)); /* FAuxHeader::Pack */
	*Buffer = bIsImportant ?
		uint8(EKnownEventUids::AuxData) : // no _UidShift !!!
		uint8(EKnownEventUids::AuxData << EKnownEventUids::_UidShift); /* FAuxHeader::Uid */
	Buffer += sizeof(FAuxHeader);

	// Write data.
	FMemory::Memcpy(Buffer, SegmentData, SegmentSize);

	AdvanceBuffer(Size);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteEnterScopeEvent()
{
	BeginEvent();
	using namespace Private::WriterProtocol;
	WriteScopeEventPrivate(uint8(EKnownEventUids::EnterScope << EKnownEventUids::_UidShift));
	EndEvent();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteLeaveScopeEvent()
{
	BeginEvent();
	using namespace Private::WriterProtocol;
	WriteScopeEventPrivate(uint8(EKnownEventUids::LeaveScope << EKnownEventUids::_UidShift));
	EndEvent();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteStampedEnterScopeEvent(uint64 Timestamp)
{
	BeginEvent();
	using namespace Private::WriterProtocol;
	constexpr uint8 Uid = uint8(EKnownEventUids::EnterScope_TB << EKnownEventUids::_UidShift);
	WriteStampedScopeEventPrivate(Uid, Timestamp);
	EndEvent();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteStampedLeaveScopeEvent(uint64 Timestamp)
{
	BeginEvent();
	using namespace Private::WriterProtocol;
	constexpr uint8 Uid = uint8(EKnownEventUids::LeaveScope_TB << EKnownEventUids::_UidShift);
	WriteStampedScopeEventPrivate(Uid, Timestamp);
	EndEvent();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteScopeEventPrivate(uint8 Uid)
{
	uint8* Buffer = GetBuffer(1);
	if (!Buffer)
	{
		SetError(TEXT("Failed to write *Scope event!"));
		return;
	}

	*Buffer = Uid;

	AdvanceBuffer(1);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WriteStampedScopeEventPrivate(uint8 Uid, uint64 Timestamp)
{
	constexpr uint32 Size = sizeof(uint64); // 1 byte (Uid) + 7 bytes (timestamp)

	uint8* Buffer = GetBuffer(Size);
	if (!Buffer)
	{
		SetError(TEXT("Failed to write *Scope_TB event!"));
		return;
	}

	const uint64 RelativeTimestamp = Timestamp - GetStartTime();
	if ((RelativeTimestamp >> 56) != 0)
	{
		SetError(TEXT("Relative timestamp is too large!"));
	}

	const uint64 Pack = (RelativeTimestamp << 8) + Uid;
	FMemory::Memcpy(Buffer, &Pack, sizeof(uint64));

	AdvanceBuffer(Size);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint16 FTraceWriter::GetByteSizeForFieldType(ETraceWriterFieldType Type)
{
	if (EnumHasAnyFlags(Type, ETraceWriterFieldType::ArrayFlag))
	{
		return 0; // arrays have variable size
	}

	switch (Type & ETraceWriterFieldType::IndexMask)
	{
	case ETraceWriterFieldType::Bool:
	case ETraceWriterFieldType::Uint8:
	case ETraceWriterFieldType::Int8:
		return 1;

	case ETraceWriterFieldType::Uint16:
	case ETraceWriterFieldType::Int16:
		return 2;

	case ETraceWriterFieldType::Uint32:
	case ETraceWriterFieldType::Int32:
	case ETraceWriterFieldType::Float32: // float
		return 4;

	case ETraceWriterFieldType::Uint64:
	case ETraceWriterFieldType::Int64:
	case ETraceWriterFieldType::Float64: // double
	case ETraceWriterFieldType::Pointer:
		return 8;

	default:
		return 0; // unknown or variable size
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint8 FTraceWriter::GetProtocolFieldFamily(ETraceWriterFieldType Type)
{
	// See UE::Trace::Protocol6::EFieldFamily (Trace\Detail\Protocols\Protocol6.h in TraceLog module).
	// { Regular, Reference, DefinitionId }

	using namespace Private::WriterProtocol;

	if ((Type & ETraceWriterFieldType::FlagsMask) == ETraceWriterFieldType::None ||
		(Type & ETraceWriterFieldType::FlagsMask) == ETraceWriterFieldType::ArrayFlag)
	{
		return EFieldFamily::Regular;
	}
	if ((Type & ETraceWriterFieldType::FlagsMask) == ETraceWriterFieldType::ReferenceFlag)
	{
		return EFieldFamily::Reference;
	}
	if ((Type & ETraceWriterFieldType::FlagsMask) == ETraceWriterFieldType::DefinitionIdFlag)
	{
		return EFieldFamily::DefinitionId;
	}
	return 0xFF; // error
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint8 FTraceWriter::GetProtocolFieldType(ETraceWriterFieldType Type)
{
	// See UE::Trace::Protocol0::EFieldType (Trace\Detail\Protocols\Protocol0.h in TraceLog module).

	using namespace Private::WriterProtocol;

	if (EnumHasAnyFlags(Type, ETraceWriterFieldType::ArrayFlag))
	{
		return uint8(EFieldType::Array) | GetProtocolFieldType(Type & ~ETraceWriterFieldType::ArrayFlag);
	}

	if (EnumHasAnyFlags(Type, ETraceWriterFieldType::ReferenceFlag))
	{
		switch (Type & (ETraceWriterFieldType::IndexMask | ETraceWriterFieldType::ReferenceFlag))
		{
			case ETraceWriterFieldType::Reference8:  return uint8(EFieldType::Uint8);
			case ETraceWriterFieldType::Reference16: return uint8(EFieldType::Uint16);
			case ETraceWriterFieldType::Reference32: return uint8(EFieldType::Uint32);
			case ETraceWriterFieldType::Reference64: return uint8(EFieldType::Uint64);
		}
		return 0;
	}

	if (EnumHasAnyFlags(Type, ETraceWriterFieldType::DefinitionIdFlag))
	{
		switch (Type & (ETraceWriterFieldType::IndexMask | ETraceWriterFieldType::DefinitionIdFlag))
		{
			case ETraceWriterFieldType::DefinitionId8:  return uint8(EFieldType::Uint8);
			case ETraceWriterFieldType::DefinitionId16: return uint8(EFieldType::Uint16);
			case ETraceWriterFieldType::DefinitionId32: return uint8(EFieldType::Uint32);
			case ETraceWriterFieldType::DefinitionId64: return uint8(EFieldType::Uint64);
		}
		return 0;
	}

	switch (Type & ETraceWriterFieldType::IndexMask)
	{
		case ETraceWriterFieldType::None:        return 0;
		case ETraceWriterFieldType::Bool:        return uint8(EFieldType::Bool); // == EFieldType::Uint8 == 0
		case ETraceWriterFieldType::Uint8:       return uint8(EFieldType::Uint8); // == 0
		case ETraceWriterFieldType::Uint16:      return uint8(EFieldType::Uint16);
		case ETraceWriterFieldType::Uint32:      return uint8(EFieldType::Uint32);
		//case ETraceWriterFieldType::Uint32_7bit: return 0; // not supported in Protocol7
		case ETraceWriterFieldType::Uint64:      return uint8(EFieldType::Uint64);
		//case ETraceWriterFieldType::Uint64_7bit: return 0; // not supported in Protocol7
		case ETraceWriterFieldType::Int8:        return uint8(EFieldType::Int8);
		case ETraceWriterFieldType::Int16:       return uint8(EFieldType::Int16);
		case ETraceWriterFieldType::Int32:       return uint8(EFieldType::Int32);
		//case ETraceWriterFieldType::Int32_7bit:  return 0; // not supported in Protocol7
		case ETraceWriterFieldType::Int64:       return uint8(EFieldType::Int64);
		//case ETraceWriterFieldType::Int64_7bit:  return 0; // not supported in Protocol7
		case ETraceWriterFieldType::Float32:     return uint8(EFieldType::Float32);
		case ETraceWriterFieldType::Float64:     return uint8(EFieldType::Float64);
		case ETraceWriterFieldType::AnsiString:  return uint8(EFieldType::AnsiString);
		case ETraceWriterFieldType::WideString:  return uint8(EFieldType::WideString);
		//case ETraceWriterFieldType::Utf8String:  return 0; // not supported in Protocol7
		case ETraceWriterFieldType::Pointer:     return uint8(EFieldType::Pointer); // == EFieldType::Uint64
		default:
			return 0;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ETraceWriterFieldType FTraceWriter::ConvertFieldType(const UE::Trace::IAnalyzer::FEventFieldInfo& FieldInfo)
{
	ETraceWriterFieldType WriterFieldType = ETraceWriterFieldType::None;

	using EType = UE::Trace::IAnalyzer::FEventFieldInfo::EType;
	const EType Type = FieldInfo.GetType();
	switch (Type)
	{
		case EType::Integer:
			if (FieldInfo.IsSigned())
			{
				switch (FieldInfo.GetTypeSize())
				{
					case 1: WriterFieldType = ETraceWriterFieldType::Int8; break;
					case 2: WriterFieldType = ETraceWriterFieldType::Int16; break;
					case 4: WriterFieldType = ETraceWriterFieldType::Int32; break;
					case 8: WriterFieldType = ETraceWriterFieldType::Int64; break;
				}
			}
			else
			{
				switch (FieldInfo.GetTypeSize())
				{
					case 1: WriterFieldType = ETraceWriterFieldType::Uint8; break;
					case 2: WriterFieldType = ETraceWriterFieldType::Uint16; break;
					case 4: WriterFieldType = ETraceWriterFieldType::Uint32; break;
					case 8: WriterFieldType = ETraceWriterFieldType::Uint64; break;
				}
			}
			if (FieldInfo.IsArray())
			{
				EnumAddFlags(WriterFieldType, ETraceWriterFieldType::ArrayFlag);
			}
			break;

		case EType::Float:
			if (FieldInfo.GetTypeSize() == 4)
			{
				WriterFieldType = ETraceWriterFieldType::Float32;
			}
			else
			{
				check(FieldInfo.GetTypeSize() == 8);
				WriterFieldType = ETraceWriterFieldType::Float64;
			}
			if (FieldInfo.IsArray())
			{
				EnumAddFlags(WriterFieldType, ETraceWriterFieldType::ArrayFlag);
			}
			break;

		case EType::AnsiString: WriterFieldType = ETraceWriterFieldType::AnsiString; break;
		case EType::WideString: WriterFieldType = ETraceWriterFieldType::WideString; break;

		case EType::Reference8:  WriterFieldType = ETraceWriterFieldType::ReferenceFlag | ETraceWriterFieldType::Uint8;  break;
		case EType::Reference16: WriterFieldType = ETraceWriterFieldType::ReferenceFlag | ETraceWriterFieldType::Uint16; break;
		case EType::Reference32: WriterFieldType = ETraceWriterFieldType::ReferenceFlag | ETraceWriterFieldType::Uint32; break;
		case EType::Reference64: WriterFieldType = ETraceWriterFieldType::ReferenceFlag | ETraceWriterFieldType::Uint64; break;
	}

	if (EnumHasAnyFlags(WriterFieldType, ETraceWriterFieldType::ReferenceFlag) &&
		FCStringAnsi::Strcmp(FieldInfo.GetName(), "DefinitionId") == 0)
	{
		WriterFieldType = ETraceWriterFieldType::DefinitionIdFlag | (WriterFieldType & ~ETraceWriterFieldType::ReferenceFlag);
	}

	return WriterFieldType;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint8* FTraceWriter::GetBuffer(uint32 RequiredDataSize)
{
	const uint32 TotalDataSize = MaxPacketHeaderSize + EventBufferDataSize + RequiredDataSize;
	if (TotalDataSize > EventBufferSize)
	{
		constexpr uint32 PageSize = 64 * 1024;
		EventBufferSize = (TotalDataSize + PageSize - 1) & ~(PageSize - 1);
		EventBuffer = (uint8*)FMemory::Realloc(EventBuffer, EventBufferSize, 0);
	}
	return EventBuffer + MaxPacketHeaderSize + EventBufferDataSize;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::AdvanceBuffer(uint32 DataSize)
{
	EventBufferDataSize += DataSize;
	check(MaxPacketHeaderSize + EventBufferDataSize <= EventBufferSize);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::BeginEvent()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::EndEvent()
{
	if (EventBufferDataSize > MaxDecodedBufferSize)
	{
		// Not enough packet space to include the current event.
		// Write a packet with the previous event(s) first.
		// The current event remains pending in the EventBuffer.
		WritePacket();
		check(EventBufferDataSize <= MaxDecodedBufferSize);
	}

	CompletedEventsDataSize = EventBufferDataSize;

	if (CompletedEventsDataSize >= FlushPacketThreshold)
	{
		WritePacket();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::WritePacket()
{
	if (CompletedEventsDataSize == 0)
	{
		return;
	}

	using namespace UE::Trace::Private;

	uint8* Data = EventBuffer + MaxPacketHeaderSize;
	uint32 Size = CompletedEventsDataSize;
	check(Size <= MaxDecodedBufferSize); // PacketSize and DecodedSize are uint16

	bool bWriteUncompressedData = false;

	// Smaller buffers usually aren't redundant enough to benefit from being
	// compressed. They often end up being larger.
	if (Size > MinEncodedBufferSize && Size <= MaxEncodedBufferSize)
	{
		if (PacketBuffer == nullptr)
		{
			PacketBuffer = (uint8*)FMemory::Malloc(MaxPacketBufferSize, 0);
		}
		FTidPacketEncoded& Packet = *(FTidPacketEncoded*)PacketBuffer;
		const int32 EncodedSize = Encode(Data, Size, Packet.Data, MaxEncodedBufferSize + EncodingOverhead);
		if (EncodedSize > 0 && uint32(EncodedSize) < Size)
		{
			Packet.PacketSize = uint16(sizeof(FTidPacketEncoded) + EncodedSize);
			Packet.ThreadId = FTidPacketBase::EncodedMarker;
			Packet.ThreadId |= uint16(CurrentThreadId & FTidPacketBase::ThreadIdMask);
			Packet.DecodedSize = uint16(Size);
			Stream.Write(&Packet, Packet.PacketSize);
		}
		else
		{
			bWriteUncompressedData = true;
		}
	}
	else
	{
		bWriteUncompressedData = true;
	}

	if (bWriteUncompressedData)
	{
		Data -= sizeof(FTidPacket);
		Size += sizeof(FTidPacket);
		FTidPacket& Packet = *(FTidPacket*)Data;
		Packet.PacketSize = uint16(Size);
		Packet.ThreadId = uint16(CurrentThreadId & FTidPacketBase::ThreadIdMask);
		Stream.Write(Data, Size);
	}

	check(EventBufferDataSize >= CompletedEventsDataSize);
	EventBufferDataSize -= CompletedEventsDataSize;
	if (EventBufferDataSize > 0)
	{
		FMemory::Memmove(EventBuffer + MaxPacketHeaderSize, EventBuffer + MaxPacketHeaderSize + CompletedEventsDataSize, EventBufferDataSize);
	}

	CompletedEventsDataSize = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::InitDefaultClock()
{
	check(!bUseCustomClock);

	CycleFrequency = FPlatformTime::SecondsToCycles64(1.0);
	TimeGetter = []() { return FPlatformTime::Cycles64(); };

	StartCycle = FPlatformTime::Cycles64();

	//std::chrono::duration_cast<std::chrono::duration<double>>(std::chrono::system_clock::now().time_since_epoch()).count()
	StartTimeSinceEpoch = FDateTime::Now().ToUnixTimestampDecimal();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceWriter::SetError(FString&& InError)
{
	Errors.Push(InError);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FString& FTraceWriter::GetLastError() const
{
	if (Errors.Num() > 0)
	{
		LastError = Errors.Pop();
	}
	else
	{
		LastError.Reset();
	}
	return LastError;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Trace
