// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"

#define UE_API TRACEANALYSIS_API

class FMessageLog;

namespace UE {
namespace Trace {

class IAnalyzer;
class IInDataStream;
	
enum class EAnalysisMessageSeverity
{
	Info,
	Warning,
	Error
};

DECLARE_DELEGATE_TwoParams(FMessageDelegate, EAnalysisMessageSeverity, FStringView);
DECLARE_DELEGATE(FOnCompletedDelegate);

/**
 * Represents the processing (e.g. analysis) of a trace stream. Instances are
 * created by constructing an FAnalysisContext object to marry an event trace
 * with how it should be analyzed. Note that the processing (and thus analysis)
 * happens on another thread.
 */
class FAnalysisProcessor
{
public:
	/** Checks if this object instance is valid and currently processing */
	UE_API bool IsActive() const;

	/** End processing a trace stream. */
	UE_API void Stop();

	/** Wait for the entire stream to have been processed and analysed. */
	UE_API void Wait();

	/** Pause or resume the processing.
	 * @param bState Pause if true, resume if false. */
	UE_API void Pause(bool bState);


						UE_API ~FAnalysisProcessor();
						FAnalysisProcessor() = default;
						UE_API FAnalysisProcessor(FAnalysisProcessor&& Rhs);
	UE_API FAnalysisProcessor&	operator = (FAnalysisProcessor&&);

private:
	friend				class FAnalysisContext;
						FAnalysisProcessor(FAnalysisProcessor&) = delete;
	FAnalysisProcessor&	operator = (const FAnalysisProcessor&) = delete;
	class				FImpl;
	FImpl*				Impl = nullptr;
};



/**
 * Used to describe how a log of trace events should be analyzed and being the
 * analysis on a particular trace stream.  
 */
class FAnalysisContext
{
public:
	/** Adds an analyzer instance that will subscribe to and receive event data
	 * from the trace stream. */
	UE_API void AddAnalyzer(IAnalyzer& Analyzer);

	/** Adds a callback to recieve important messages. */
	UE_API void SetMessageDelegate(FMessageDelegate Delegate);

	/** Adds a callback that is invoked on the analysis thread when processing completes. */
	UE_API void SetOnCompletedDelegate(FOnCompletedDelegate Delegate);

	/** Creates and starts analysis returning an FAnalysisProcessor instance which
	 * represents the analysis and affords some control over it.
	 * @param DataStream Input stream of trace log data to be analysed. */
	UE_API FAnalysisProcessor Process(IInDataStream& DataStream);

private:
	TArray<IAnalyzer*>	Analyzers;
	FMessageDelegate	OnMessage;
	FOnCompletedDelegate OnCompleted;
};

} // namespace Trace
} // namespace UE

#undef UE_API
