// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "TraceAnalysisDebug.h"
#include "Trace/Detail/Protocol.h"
#include "Trace/Trace.h"

#include <type_traits>

#define UE_API TRACEANALYSIS_API

namespace UE {
namespace Trace {

/**
 * Interface that users implement to analyze the events in a trace. Analysis
 * works by subscribing to events by name along with a user-provider "route"
 * identifier. The IAnalyzer then receives callbacks when those events are
 * encountered along with an interface to query the value of the event's fields.
 *
 * To analyze a trace, concrete IAnalyzer-derived objects are registered with a
 * FAnalysisContext which is then asked to launch and coordinate the analysis.
 */
class IAnalyzer
{
public:
	struct FInterfaceBuilder
	{
		/** Subscribe to an event required for analysis.
		 * @param RouteId User-provided identifier for this event subscription.
		 * @param Logger Name of the logger that emits the event.
		 * @param Event Name of the event to subscribe to.
		 * @param bScoped Route scoped events. */
		virtual void RouteEvent(uint16 RouteId, const ANSICHAR* Logger, const ANSICHAR* Event, bool bScoped=false) = 0;

		/** Subscribe to all events from a particular logger.
		 * @param RouteId User-provided identifier for this event subscription.
		 * @param Logger Name of the logger that emits the event.
		 * @param bScoped Route scoped events. */
		virtual void RouteLoggerEvents(uint16 RouteId, const ANSICHAR* Logger, bool bScoped=false) = 0;

		/** Subscribe to all events in the trace stream being analyzed.
		 * @param RouteId User-provided identifier for this event subscription.
		 * @param bScoped Route scoped events. */
		virtual void RouteAllEvents(uint16 RouteId, bool bScoped=false) = 0;
	};

	struct FOnAnalysisContext
	{
		FInterfaceBuilder& InterfaceBuilder;
	};

	struct FEventFieldInfo
	{
		enum class EType { None, Integer, Float, AnsiString, WideString, Reference8, Reference16, Reference32, Reference64 };

		/** Returns the name of the field. */
		UE_API const ANSICHAR* GetName() const;

		/** What type of field is this? */
		UE_API EType GetType() const;

		/** Is this field an array-type field? */
		UE_API bool IsArray() const;

		/** Is this field signed (only relevant for integer types) */
		UE_API bool IsSigned() const;

#if UE_TRACE_ANALYSIS_DEBUG_API
		/** Offset from the start of the event to this field's data. */
		UE_API uint32 GetOffset() const;
#endif // UE_TRACE_ANALYSIS_DEBUG_API

		/** Gets the size in bytes for this field. */
		UE_API uint8 GetSize() const;

		/** Gets the size in bytes for the type of this field. */
		UE_API uint8 GetTypeSize() const;

		/** Gets the Reference Uid for this field. */
		UE_API uint32 GetRefUid() const;
	};

	struct FEventFieldHandle
	{
		bool	IsValid() const { return Detail >= 0; }
		int32	Detail;
	};

	enum class EEventFlags : uint8
	{
		None        = 0,
		Important   = 1 << 0,
		MaybeHasAux = 1 << 1,
		NoSync      = 1 << 2,
		Definition  = 1 << 3,
	};

	struct FEventTypeInfo
	{
		/** Each event is assigned a unique ID when logged. Note that this is not
		 * guaranteed to be the same for the same event from one trace to the next. */
		UE_API uint32 GetId() const;

		/** Returns the event's flags. */
		UE_API EEventFlags GetFlags() const;
		inline bool IsImportant() const  { return ((uint8)GetFlags() & (uint8)EEventFlags::Important) != 0; }
		inline bool IsSync() const       { return ((uint8)GetFlags() & (uint8)EEventFlags::NoSync) == 0; }
		inline bool IsNoSync() const     { return ((uint8)GetFlags() & (uint8)EEventFlags::NoSync) != 0; }
		inline bool IsDefinition() const { return ((uint8)GetFlags() & (uint8)EEventFlags::Definition) != 0; }
		inline bool MaybeHasAux() const  { return ((uint8)GetFlags() & (uint8)EEventFlags::MaybeHasAux) != 0; }

		/** The name of the event. */
		UE_API const ANSICHAR* GetName() const;

		/** Returns the logger name the event is associated with. */
		UE_API const ANSICHAR* GetLoggerName() const;

		/** Returns the base size of the event. */
		UE_API uint32 GetSize() const;

		/** The number of member fields this event has. */
		UE_API uint32 GetFieldCount() const;

		/** By-index access to fields' type information. */
		UE_API const FEventFieldInfo* GetFieldInfo(uint32 Index) const;

		/** Returns the field index or -1 (if the event does not contains a field with the specified name). */
		UE_API int32 GetFieldIndex(const ANSICHAR* FieldName) const;

		/** Returns a handle that can used to access events' fields. There is
		 * loose validation via ValueType, but one should still exercise caution
		 * when reading fields with handles.
		 * @param ValueType The intended type that the field will be interpreted as */
		template <typename ValueType>
		FEventFieldHandle GetFieldHandle(const ANSICHAR* FieldName) const;

		/** Returns a handle without specifying type. This should only be used in circumstances
		 * where the field is treated untyped data.
		 * @param Index Index of field.
		 */
		UE_API FEventFieldHandle GetFieldHandleUnchecked(uint32 Index) const;

	private:
		UE_API FEventFieldHandle GetFieldHandleImpl(const ANSICHAR*, int16&) const;
	};

	struct FArrayReader
	{
		/* Returns the number of elements in the array. */
		UE_API uint32 Num() const;

#if UE_TRACE_ANALYSIS_DEBUG_API
		/* Returns the pointer to the raw data array. */
		UE_API const uint8* GetRawData() const;

		/* Returns the size in bytes of the raw data array. */
		UE_API uint32 GetRawDataSize() const;

		/* Returns the size and type of an array element. */
		UE_API int8 GetSizeAndType() const;
#endif // UE_TRACE_ANALYSIS_DEBUG_API

	protected:
		UE_API const void* GetImpl(uint32 Index, int8& SizeAndType) const;
	};

	template <typename ValueType>
	struct TArrayReader
		: public FArrayReader
	{
		/* Returns a element from the array an the given index or zero if Index
		 * is out of bounds. */
		ValueType operator [] (uint32 Index) const;

		/** Get a pointer to the contiguous array data */
		const ValueType* GetData() const;
	};

	enum class EStyle : uint32
	{
		Normal,
		EnterScope,
		LeaveScope,
	};

	struct FOnEventContext;

	struct FEventData
	{
		/** Returns an object describing the underlying event's type. */
		UE_API const FEventTypeInfo& GetTypeInfo() const;

		/** Queries the value of a field of the event. It is not necessary to match
		 * ValueType to the type in the event.
		 * @param FieldName The name of the event's field to get the value for.
		 * @param Default Return this value if the given field was not found.
		 * @return Value of the field (coerced to ValueType) if found, otherwise 0. */
		template <typename ValueType> ValueType GetValue(const ANSICHAR* FieldName, ValueType Default=ValueType(0)) const;
		template <typename ValueType> ValueType GetValue(FEventFieldHandle FieldHandle) const;

		/** Returns an object for reading data from an array-type field. A valid
		 * array reader object will always be return even if no field matching the
		 * given name was found.
		 * @param FieldName The name of the event's field to get the value for. */
		template <typename ValueType> const TArrayReader<ValueType>& GetArray(const ANSICHAR* FieldName) const;

		/** Returns an array view for reading data from an array-type field. A valid
		 * array view will always be returned even if no field matching the
		 * given name was found.
		 * @param FieldName The name of the event's field to get the value for. */
		template <typename ValueType> TArrayView<const ValueType> GetArrayView(const ANSICHAR* FieldName) const;

		/** Return the value of a string-type field. The view-type prototypes
		 * must match the underlying string type while the FString-variant is
		 * agnostic of the field's encoding.
		  * @param Out Destination object for the field's value.
		  * @return True if the field was found. */
		UE_API bool GetString(const ANSICHAR* FieldName, FAnsiStringView& Out) const;
		UE_API bool GetString(const ANSICHAR* FieldName, FWideStringView& Out) const;
		UE_API bool GetString(const ANSICHAR* FieldName, FString& Out) const;

		/** Returns a value of a reference field.
		 * @param FieldName Name of field
		 * @return Reference value
		 */
		template<typename DefinitionType>
		TEventRef<DefinitionType> GetReferenceValue(const ANSICHAR* FieldName) const;

		template<typename DefinitionType>
		TEventRef<DefinitionType> GetReferenceValue(uint32 FieldIndex) const;

		/** If this is a spec event, gets the unique Id for this spec.
		 * @return A valid spec id if the event is valid, otherwise an empty id.
		 */
		template<typename DefinitionType> TEventRef<DefinitionType> GetDefinitionId() const;

		/** The size of the event in uncompressed bytes excluding the header */
		UE_API uint32 GetSize() const;

		/** Serializes the event to Cbor object.
		 * @param Recipient of the Cbor serialization. Data is appended to Out. */
		UE_API void SerializeToCbor(TArray<uint8>& Out) const;

		/**
		 * Returns the raw pointer to a field value
		 * @param Handle Handle to field
		 * @return Untyped pointer to field value
		 */
		UE_API const void* GetValueRaw(FEventFieldHandle Handle) const;

		/** Returns the event's attachment. Not that this will always return an
		 * address but if the event has no attachment then reading from that
		 * address if undefined. */
		UE_API const uint8* GetAttachment() const;

		/** Returns the size of the events attachment, or 0 if none. */
		UE_API uint32 GetAttachmentSize() const;

#if UE_TRACE_ANALYSIS_DEBUG_API
		/** Provides a pointer to the raw event data. */
		UE_API const uint8* GetRawPointer() const;

		/** Returns the size of the raw event data (including attachment). */
		UE_API uint32 GetRawSize() const;

		/** Returns the total uncompressed size of the aux data (including size of aux headers and terminator), in bytes. */
		UE_API uint32 GetAuxSize() const;

		/** Returns the total uncompressed size of the event, in bytes (including headers and aux data). */
		UE_API uint32 GetTotalSize(IAnalyzer::EStyle Style, const IAnalyzer::FOnEventContext& Context, uint32 ProtocolVersion = 7) const;
#endif // UE_TRACE_ANALYSIS_DEBUG_API

	private:
		UE_API bool IsDefinitionImpl(uint32& OutTypeId) const;
		UE_API const void* GetReferenceValueImpl(const char* FieldName, uint16& OutSizeType, uint32& OutTypeUid) const;
		UE_API const void* GetReferenceValueImpl(uint32 FieldIndex, uint32& OutTypeUid) const;
		UE_API const void* GetValueImpl(const ANSICHAR* FieldName, int8& SizeAndType) const;
		UE_API const FArrayReader* GetArrayImpl(const ANSICHAR* FieldName) const;
	};

	struct FThreadInfo
	{
		/* Returns the trace-specific id for the thread */
		UE_API uint32 GetId() const;

		/* Returns the system id for the thread. Because this may not be known by
		 * trace and because IDs can be reused by the system, relying on the value
		 * of this is discouraged. */
		UE_API uint32 GetSystemId() const;

		/* Returns a hint for use when sorting threads. */
		UE_API int32 GetSortHint() const;

		/* Returns the thread's name or an empty string */
		UE_API const ANSICHAR* GetName() const;

		/* Returns the name of the group a thread has been assigned ti, or an empty string */
		UE_API const ANSICHAR* GetGroupName() const;
	};

	struct FEventTime
	{
		/** Returns the integer timestamp for the event or zero if there no associated timestamp. */
		UE_API uint64 GetTimestamp() const;

		/** Time of the event in seconds (from the start of the trace). Zero if there is no time for the event. */
		UE_API double AsSeconds() const;

		/** Returns a timestamp for the event compatible with FPlatformTime::Cycle64(), or zero if the event has no timestamp. */
		UE_API uint64 AsCycle64() const;

		/** Returns a FPlatformTime::Cycle64() value as seconds relative to the start of the trace. */
		UE_API double AsSeconds(uint64 Cycles64) const;

		/** As AsSeconds(Cycles64) but absolute. */
		UE_API double AsSecondsAbsolute(int64 DurationCycles64) const;

		/** Returns the double timestamp in unix for when the trace started. */
		UE_API double GetBaseDateTimestamp() const;
	};

	struct FOnEventContext
	{
		const FThreadInfo&	ThreadInfo;
		const FEventTime&	EventTime;
		const FEventData&	EventData;
	};

	virtual ~IAnalyzer() = default;

	/** Called when analysis of a trace is beginning. Analyzer implementers can
	 * subscribe to the events that they are interested in at this point
	 * @param Context Contextual information and interface for subscribing to events. */
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context)
	{
	}

	/** Indicates that the analysis of a trace log has completed and there are no
	 * further events */
	virtual void OnAnalysisEnd()
	{
	}

	/** Called when information about a thread has been updated. It is entirely
	 * possible that this might get called more than once for a particular thread
	 * if its details changed.
	 * @param ThreadInfo Describes the thread whose information has changed. */
	virtual void OnThreadInfo(const FThreadInfo& ThreadInfo)
	{
	}

	/** When a new event type appears in the trace stream, this method is called
	 * if the event type has been subscribed to.
	 * @param RouteId User-provided identifier for this event subscription.
	 * @param TypeInfo Object describing the new event's type.
	 * @return This analyzer is removed from the analysis session if false is returned. */
	virtual bool OnNewEvent(uint16 RouteId, const FEventTypeInfo& TypeInfo)
	{
		return true;
	}

	/** For each event subscribed to in OnAnalysisBegin(), the analysis engine
	 * will call this method when those events are encountered in a trace log
	 * @param RouteId User-provided identifier given when subscribing to a particular event.
	 * @param Style Indicates the style of event. Note that EventData is *undefined* if the style is LeaveScope!
	 * @param Context Access to the instance of the subscribed event.
	 * @return This analyzer is removed from the analysis session if false is returned. */
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
	{
		return true;
	}

	/** Called when information about trace transport and trace protocol versions are available.
	 * This is called after OnAnalysisBegin(), but before any OnNewEvent() or OnEvent() calls.
	 * @param TransportVersion The version of the trace transport (see Trace\Detail\Transport.h)
	 * @param ProtocolVersion The version of the trace protocol (see Trace\Detail\Protocols\*.h) */
	virtual void OnVersion(uint32 TransportVersion, uint32 ProtocolVersion)
	{
	}

private:
	template <typename ValueType> static ValueType CoerceValue(const void* Addr, int8 SizeAndType);
};

////////////////////////////////////////////////////////////////////////////////
template <typename ValueType>
IAnalyzer::FEventFieldHandle IAnalyzer::FEventTypeInfo::GetFieldHandle(const ANSICHAR* FieldName) const
{
	int16 SizeAndType;
	FEventFieldHandle Handle = GetFieldHandleImpl(FieldName, SizeAndType);
	if (std::is_floating_point<ValueType>::value)
	{
		checkf((SizeAndType < 0), TEXT("Field is not a float-type field"));
	}
	else
	{
		checkf(SizeAndType >= sizeof(ValueType), TEXT("Field is to small to read as hinted type ValueType"));
	}
	return Handle;
}

////////////////////////////////////////////////////////////////////////////////
template <typename ValueType>
ValueType IAnalyzer::CoerceValue(const void* Addr, int8 SizeAndType)
{
	using integral8 = std::conditional_t<std::is_signed_v<ValueType>, int8, uint8>;
	using integral16 = std::conditional_t<std::is_signed_v<ValueType>, int16, uint16>;
	using integral32 = std::conditional_t<std::is_signed_v<ValueType>, int32, uint32>;
	using integral64 = std::conditional_t<std::is_signed_v<ValueType>, int64, uint64>;
	switch (SizeAndType)
	{
	case -4: return ValueType(*(const float*)(Addr));
	case -8: return ValueType(*(const double*)(Addr));
	case  1: return ValueType(*(const integral8*)(Addr));
	case  2: return ValueType(*(const integral16*)(Addr));
	case  4: return ValueType(*(const integral32*)(Addr));
	case  8: return ValueType(*(const integral64*)(Addr));
	default: return ValueType(0);
	}
}

////////////////////////////////////////////////////////////////////////////////
template <typename ValueType>
ValueType IAnalyzer::FEventData::GetValue(const ANSICHAR* FieldName, ValueType Default) const
{
	int8 FieldSizeAndType;
	if (const void* Addr = GetValueImpl(FieldName, FieldSizeAndType))
	{
		return CoerceValue<ValueType>(Addr, FieldSizeAndType);
	}
	return Default;
}

////////////////////////////////////////////////////////////////////////////////
template <typename ValueType>
ValueType IAnalyzer::FEventData::GetValue(FEventFieldHandle FieldHandle) const
{
	const uint8* EventDataPtr = *(const uint8**)this;
	return *(ValueType*)(EventDataPtr + FieldHandle.Detail);
}

////////////////////////////////////////////////////////////////////////////////
template <typename ValueType>
const IAnalyzer::TArrayReader<ValueType>& IAnalyzer::FEventData::GetArray(const ANSICHAR* FieldName) const
{
	const FArrayReader* Base = GetArrayImpl(FieldName);
	return *(TArrayReader<ValueType>*)(Base);
}

////////////////////////////////////////////////////////////////////////////////
template <typename ValueType>
TArrayView<const ValueType> IAnalyzer::FEventData::GetArrayView(const ANSICHAR* FieldName) const
{
	const TArrayReader<ValueType>& ArrayReader = GetArray<ValueType>(FieldName);
	return TArrayView<const ValueType>(ArrayReader.GetData(), ArrayReader.Num());
}

////////////////////////////////////////////////////////////////////////////////
template <typename ValueType>
ValueType IAnalyzer::TArrayReader<ValueType>::operator [] (uint32 Index) const
{
	int8 ElementSizeAndType;
	if (const void* Addr = GetImpl(Index, ElementSizeAndType))
	{
		return CoerceValue<ValueType>(Addr, ElementSizeAndType);
	}
	return ValueType(0);
}

////////////////////////////////////////////////////////////////////////////////
template <typename ValueType>
const ValueType* IAnalyzer::TArrayReader<ValueType>::GetData() const
{
	int8 ElementSizeAndType;
	const void* Addr = GetImpl(0, ElementSizeAndType);

	if (Addr == nullptr || sizeof(ValueType) != abs(ElementSizeAndType))
	{
		return nullptr;
	}

	return (const ValueType*)Addr;
}

////////////////////////////////////////////////////////////////////////////////
template<typename DefinitionType>
TEventRef<DefinitionType> IAnalyzer::FEventData::GetDefinitionId() const
{
	uint32 TypeUid;
	if (IsDefinitionImpl(TypeUid))
	{
		//todo: Emit warning when trying to access id of incorrect type?
		return MakeEventRef(GetValue<DefinitionType>("DefinitionId", 0), TypeUid);
	}
	return MakeEventRef<DefinitionType>(0,0);
}

////////////////////////////////////////////////////////////////////////////////
template<typename DefinitionType>
TEventRef<DefinitionType> IAnalyzer::FEventData::GetReferenceValue(const ANSICHAR* FieldName) const
{
	uint32 TypeUid;
	uint16 SizeAndType;
	const void* Value = GetReferenceValueImpl(FieldName, SizeAndType, TypeUid);
	if (Value)
	{
		return MakeEventRef<DefinitionType>(CoerceValue<DefinitionType>(Value, static_cast<int8>(SizeAndType)), TypeUid);
	}
	return MakeEventRef<DefinitionType>(0,0);
}

////////////////////////////////////////////////////////////////////////////////
template <typename DefinitionType>
TEventRef<DefinitionType> IAnalyzer::FEventData::GetReferenceValue(uint32 FieldIndex) const
{
	uint32 RefTypeUid;
	DefinitionType* Id = (DefinitionType*) GetReferenceValueImpl(FieldIndex, RefTypeUid);
	return MakeEventRef<DefinitionType>(*Id, RefTypeUid);
}

} // namespace Trace
} // namespace UE

#undef UE_API
