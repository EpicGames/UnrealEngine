// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#define UE_API TRACEANALYSIS_API

class FSocket;
class FInternetAddr;

namespace UE {
namespace Trace {

/**
 * Simple helper class for communicating with trace system of a running instance.
 */
class FControlClient
{
public:
	/** The destructor will ensure disconnection the FControlClient class
	 * goes out of scope. */
    UE_API ~FControlClient();

	/** Initiates a connection to a runtime instance to control.
	 * @param Host IP address or host name of where the runtime is to be found.
	 * @returns True if a connection was made successfully. */
    UE_API bool Connect(const TCHAR* Host, uint16 Port=1985);

	/** Initiates a connection to a runtime instance to control.
	 * @param Fully qualified FInternetAddr object of the runtime's location.
	 * @returns True if a connection was made successfully. */
	UE_API bool Connect(const FInternetAddr& Address);

	/** Disconnects the client from the runtime. */
    UE_API void Disconnect();

	/** Returns true if the client is currently connected to a runtime */
    UE_API bool IsConnected() const;

	/** Tells the runtime to attempt to record its trace event log to a socket.
	 * @param Host Host IP address that the runtime should send events to. */
    UE_API void SendSendTo(const TCHAR* Host);

	/** Tells the runtime to attempt to record its trace event log to a file.
	 * @param Path File system path that the runtime should write events to. */
    UE_API void SendWriteTo(const TCHAR* Path);

	/** Tells the runtime to stop tracing. */
    UE_API void SendStop();

	/**
	 * Toggles one or more channels on and off. A channel or a comma separated list
	 * of channels can be controlled.
	 * @param Channels A single channel name or a comma separated list of channel names.
	 * @param bState True to enable channel(s), false to disable.
	 */
	UE_API void SendToggleChannel(const TCHAR* Channels, bool bState = true);

	/** Sends a raw command to the instance.
	 * @param Command The command to send to the runtime (CRLF terminator not required). */
    UE_API void Send(const TCHAR* Command);

private:
    void        FormatAndSend(const TCHAR* Format, ...);
    void        Send(const uint8* Data, int Length);
    FSocket*    Socket = nullptr;
};

} // namespace Trace
} // namespace UE

#undef UE_API
