// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/Platform.h"
#include "HAL/Runnable.h"
#include "Templates/UniquePtr.h"

#define UE_API TRACEANALYSIS_API

class IFileHandle;
class FSocket;

namespace UE::Trace
{

class IInDataStream
{
public:
	virtual ~IInDataStream() = default;

	/**
	 * Reads bytes from the stream.
	 *
	 * @param Data - The pointer to buffer to read into
	 * @param Size - The maximum size that can be read in bytes
	 * @returns the number of bytes read from the stream. Zero indicates end of stream and negative values indicate errors.
	 */
	virtual int32 Read(void* Data, uint32 Size) = 0;

	/**
	 * Closes the stream.
	 * Reading from a closed stream is considered an error.
	 */
	virtual void Close() {}

	/**
	 * Waits for the the stream to be ready to read from.
	 * Some streams may need to establish the data stream before reading can begin.
	 * A stream may not block indefinitely.
	 *
	 * @returns if the stream is ready to be read from
	 */
	virtual bool WaitUntilReady() { return true; }
};

/*
* An implementation of IInDataStream that reads from a file on disk.
*/
class FFileDataStream : public IInDataStream
{
public:
	UE_API FFileDataStream();
	UE_API virtual ~FFileDataStream() override;

	/*
	* Open the file.
	*
	* @param Path - The path to the file
	* @param bAllowWrite - If true, other processes may write to the file while it is open
	* @returns true if the file was opened successfully
	*/
	UE_API bool Open(const TCHAR* Path, bool bAllowWrite = false);

	UE_API virtual int32 Read(void* Data, uint32 Size) override;
	UE_API virtual void Close() override;

private:
	TUniquePtr<IFileHandle> Handle;
	uint64 Remaining;
};

/**
 * Creates a stream to directly consume a trace stream from the tracing application.
 * Sets up a listening socket and stream is not considered ready until a connection is made.
 * If a valid file writer is provided during construction, the trace data will also be written to disk.
 */
class FDirectSocketStream : public IInDataStream, public FRunnable
{
public:
	UE_API FDirectSocketStream();
	UE_API FDirectSocketStream(TUniquePtr<FArchiveFileWriterGeneric>&& InFileWriter);
	UE_API virtual ~FDirectSocketStream() override;

	/**
	 * Initiates listening sockets.
	 * Must be called before attempting to read from the stream.
	 * @param Port - The specific port number to connect to or 0 to use any available port
	 * @returns the actual port number used for listening
	 */
	UE_API uint16 StartListening(uint16 Port = 0);

	/**
	 * Binds this instance to an existing connection that has already been established.
	 * Using this method allows you to service connections off a single listen port, then
	 * create instances of FDirectSocketStream for each connection.
	 * @param ExistingConnectedSocket is the existing socket that will transfer its ownership to this instance, even if the function fails.
	 * @returns true if the binding was successful. Note: Even if it returns false, you will have lost ownership of ExistingConnectedSocket.
	 */
	UE_API bool BindToExistingConnection(FSocket&& ExistingConnectedSocket);

	UE_API virtual int32 Read(void* Data, uint32 Size) override;

private:
	enum
	{
		DefaultPort = 1986,			// Default port to use.
		MaxPortAttempts = 16,		// How many port increments are tried if fail to bind default.
		MaxQueuedConnections = 4,	// Size of connection queue.
	};

	// IInDataStream interface
	UE_API virtual bool WaitUntilReady() override;
	UE_API virtual void Close() override;

	// FRunnable interface
	UE_API virtual uint32 Run() override;
	UE_API virtual void Stop() override;

	void Accept();
	bool CreateSocket(uint16 Port);

private:
	TUniquePtr<struct FDirectSocketContext> AsioContext;
	TUniquePtr<class FTraceDataStream> InternalStream;
	TUniquePtr<class FRunnableThread> ListeningThread;
	TUniquePtr<FArchiveFileWriterGeneric> FileWriter;
	FEvent* ConnectionEvent;
};

} // namespace UE::Trace

#undef UE_API
