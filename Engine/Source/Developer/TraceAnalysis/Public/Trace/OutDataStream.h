// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/FileManagerGeneric.h"
#include "Templates/UniquePtr.h"

#define UE_API TRACEANALYSIS_API

class IFileHandle;

namespace UE::Trace
{

class IOutDataStream
{
public:
	virtual ~IOutDataStream() = default;

	/**
	 * Writes bytes to the stream.
	 *
	 * @param Data - The pointer to the data buffer to read from and write to the stream
	 * @param Size - The number of bytes to read from the Data buffer and to write to the stream
	 * @returns the number of bytes written to the stream. Negative values indicate errors.
	 */
	virtual int32 Write(void* Data, uint32 Size) = 0;

	/**
	 * Closes the stream.
	 * Writing to a closed stream is considered an error.
	 */
	virtual void Close() {}

	/**
	 * Waits for the stream to be ready to write to.
	 * Some streams may need to establish the data stream before writing can begin.
	 * A stream may not block indefinitely.
	 *
	 * @returns if the stream is ready to be write to
	 */
	virtual bool WaitUntilReady() { return true; }
};

/*
 * An implementation of IOutDataStream that writes to a file on disk.
 */
class FFileOutDataStream : public IOutDataStream
{
public:
	UE_API FFileOutDataStream();
	UE_API virtual ~FFileOutDataStream() override;

	/*
	* Open the file.
	*
	* @param Path - The path to the file
	* @returns true if the file was opened successfully
	*/
	UE_API bool Open(const TCHAR* Path);

	UE_API virtual int32 Write(void* Data, uint32 Size) override;
	UE_API virtual void Close() override;

private:
	TUniquePtr<IFileHandle> Handle;
};

} // namespace UE::Trace

#undef UE_API
