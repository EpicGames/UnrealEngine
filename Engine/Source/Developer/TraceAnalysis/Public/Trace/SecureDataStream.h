// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStream.h"
#include "Containers/AnsiString.h"
#include "CoreFwd.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/Platform.h"
#include "HAL/Runnable.h"
#include "HAL/Runnable.h"
#include "Containers/SpscQueue.h"
#include "Templates/UniquePtr.h"

#define UE_API TRACEANALYSIS_API

// Secure socket stream support depends on SSL support in ASIO
#define UE_TRACE_ANALYSIS_SSS_ENABLED WITH_SHARED_ASIO_SSL_SUPPORT

// Enables experimental buffered receiving path
// for FSecureSocketStream
#define UE_TRACE_ANALYSIS_SSS_BUFFERED 1

#if UE_TRACE_ANALYSIS_SSS_ENABLED 

namespace UE::Trace
{

struct FSecureStreamPSKCallback;

class FSecureSocketStream : public UE::Trace::IInDataStream, public ::FRunnable
{
public:
	UE_API FSecureSocketStream();
	UE_API FSecureSocketStream(TUniquePtr<FArchiveFileWriterGeneric>&& InFileWriter);
	UE_API virtual ~FSecureSocketStream() override;

	/**
	 * Connect to a tracing host. Specify a authorization token and an optional trusted CA certificate file.
	 * @param Host Hostname to connect to.
	 * @param Port Port to connect to.
	 * @param AuthToken Authorization token to send.
	 * @param CertificatePath Optional path to trusted root certificate
	 * @return True on success, false if not.
	 */
	UE_API bool ConnectWithCertificate(FStringView Host, uint16 Port, TConstArrayView<uint8> AuthToken, FStringView CertificatePath);

	/**
	 * Connect to a tracing host. Specify a authorization token and an optional trusted in-memory CA certificate.
	 * @param Host Hostname to connect to.
	 * @param Port Port to connect to.
	 * @param AuthToken Authorization token to send.
	 * @param Certificate A self-signed root certificate in DER format.
	 * @return True on success, false if not.
	 */
	UE_API bool ConnectWithSelfSigned(FStringView Host, uint16 Port, TConstArrayView<uint8> AuthToken, TConstArrayView<uint8> Certificate);

	/**
	 * Connect to a tracing host. Specify an authorization token used for pre-shared key verification.
	 * @param Host Hostname to connect to.
	 * @param Port Port to connect to.
	 * @param PreSharedKey Key to use in secure handshake
	 * @param Identity Identity to use in secure handshake
	 * @return True on success, false if not.
	 */
	UE_API bool ConnectWithPreSharedKey(FStringView Host, uint16 Port, TConstArrayView<uint8> PreSharedKey, FAnsiStringView Identity);


	UE_API virtual int32 Read(void* Data, uint32 Size) override;

private:
	enum
	{
		DefaultPort = 1986,			// Default port to use.
		MaxPortAttempts = 16,		// How many port increments are tried if fail to bind default.
		MaxQueuedConnections = 4,	// Size of connection queue.
	};

	enum class EState : uint8 
	{
		None = 0,
		Closed,
		Connecting,
		Handshaking,
		Authenticating,
		Open,
		Failed = 0xff
	};

	bool ConnectInternal(FStringView Host, uint16 Port, TConstArrayView<uint8> AuthToken);

	// IInStream interface
	UE_API virtual bool WaitUntilReady() override;
	UE_API virtual void Close() override;

	// FRunnable interface
	UE_API virtual uint32 Run() override;
	UE_API virtual void Stop() override;

	// Async data functions
	void AsyncHandshake();
	void AsyncAuthenticate(int32 Offset = 0);
#if UE_TRACE_ANALYSIS_SSS_BUFFERED
	void AsyncRecieveData();
#endif

	bool LoadCertificateAuthority(FStringView InCertificatePath);
	bool SetCertificateAuthority(TConstArrayView<uint8> CA);
	bool SetPreSharedKey();
	void SetState(EState NewState, bool bTriggerReady);

private:

	enum class ESecureMode : uint8 
	{
		None,
		Certificates,
		PreSharedKeys
	};
	
	class FData : public TArray<uint8>
	{
	public:
		FData(uint8* Data, int32 Size)
			: TArray<uint8>(TArrayView<uint8>(Data, Size))
		{}
	};

	TUniquePtr<struct FSecureSocketContext> AsioContext;
	TUniquePtr<class FRunnableThread> IoThread;
	FEvent* ReadyToReadEvent;
	TUniquePtr<class FArchiveFileWriterGeneric> FileWriter;
	EState State;
	TArray<uint8> AuthToken;
	FAnsiString Identity;
	ESecureMode Mode = ESecureMode::None;
#if UE_TRACE_ANALYSIS_SSS_BUFFERED
	static constexpr int32 BufferCapacity = 512 * 1024;
	TArray<uint8> ReadBuffer;
	uint32 BufferedBytes = 0u;
	TSpscQueue<FData> ReadQueue;
#endif

	friend FSecureStreamPSKCallback;
};

} // namespace UE::Trace

#endif // UE_TRACE_ANALYSIS_SSS_ENABLED 

#undef UE_API
