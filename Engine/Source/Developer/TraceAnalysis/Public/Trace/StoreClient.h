// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "HAL/Platform.h"
#include "Templates/UniquePtr.h"
#include "Trace/DataStream.h"

#define UE_API TRACEANALYSIS_API

namespace UE {
namespace Trace {

class IInDataStream;

////////////////////////////////////////////////////////////////////////////////
class FStoreClient
{
public:
	struct FStatus
	{
		UE_API FUtf8StringView				GetStoreDir() const;
		UE_API uint32						GetRecorderPort() const;
		UE_API uint32						GetStorePort() const;
		UE_API bool							GetSponsored() const;
		UE_API uint32						GetChangeSerial() const;
		UE_API uint32						GetSettingsSerial() const;
		UE_API void							GetWatchDirectories(TArray<FString>& OutDirs) const;
	};

	struct FVersion
	{
		UE_API uint32						GetMajorVersion() const;
		UE_API uint32						GetMinorVersion() const;
		UE_API FUtf8StringView				GetConfiguration() const;
	};

	struct FTraceInfo
	{
		UE_API FUtf8StringView				GetName() const;
		UE_API uint32						GetId() const;
		UE_API uint64						GetSize() const;
		UE_API uint64						GetTimestamp() const;
		UE_API FUtf8StringView				GetUri() const;
		UE_API FUtf8StringView				GetMetadataUri() const;
	};

	struct FTraceData
		: public TUniquePtr<IInDataStream>
	{
		using TUniquePtr<IInDataStream>::TUniquePtr;
	};

											~FStoreClient() = default;
	UE_DEPRECATED(5.8, "Connect has been deprecated. Please use Create which returns a TUniquePtr instead.")
	static UE_API FStoreClient*				Connect(const TCHAR* Host, uint32 Port = 0);
	static UE_API TUniquePtr<FStoreClient>  Create(const TCHAR* Host, uint32 Port = 0);
	UE_API bool								Reconnect(const TCHAR* Host, uint32 Port);
	UE_API void								operator delete (void* Addr);
	UE_API bool								IsValid() const;
	UE_API uint32							GetStoreAddress() const;
	UE_API uint32							GetStorePort() const;
	UE_API const FStatus*					GetStatus() const;
	UE_API const FVersion*					GetVersion() const;
	UE_API uint32							GetTraceCount() const;
	UE_API const FTraceInfo*				GetTraceInfo(uint32 Index) const;
	UE_API const FTraceInfo*				GetTraceInfoById(uint32 Id) const;
	UE_API FTraceData						ReadTrace(uint32 Id) const;
	UE_API bool								SetStoreDirectories(const TCHAR* StoreDir, const TArray<FString>& AddWatchDirs, const TArray<FString>& RemoveWatchDirs);
	UE_API bool								SetSponsored(bool bSponsored);

	struct FSessionInfo
	{
		UE_API uint32						GetId() const;
		UE_API uint32						GetTraceId() const;
		UE_API uint32						GetIpAddress() const;
		UE_API uint32						GetControlPort() const;
	};
	UE_API uint32							GetSessionCount() const;
	UE_API const FSessionInfo*				GetSessionInfo(uint32 Index) const;
	UE_API const FSessionInfo*				GetSessionInfoById(uint32 Id) const;
	UE_API const FSessionInfo*				GetSessionInfoByTraceId(uint32 TraceId) const;
	UE_API const FSessionInfo*				GetSessionInfoByGuid(const FGuid& TraceGuid) const;

private:
											FStoreClient() = default;
											FStoreClient(const FStoreClient&) = delete;
											FStoreClient(const FStoreClient&&) = delete;
	void									operator = (const FStoreClient&) = delete;
	void									operator = (const FStoreClient&&) = delete;
};

} // namespace Trace
} // namespace UE

#undef UE_API
