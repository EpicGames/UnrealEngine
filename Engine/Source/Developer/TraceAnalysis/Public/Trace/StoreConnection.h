// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "Templates/UniquePtr.h"

#define UE_API TRACEANALYSIS_API

namespace UE::Trace
{

class FStoreClient;

class FStoreConnection
{
public:
	UE_API FStoreConnection();
	UE_API virtual ~FStoreConnection();

	UE_API bool ConnectToStore(const TCHAR* Host, uint32 Port = 0);

	/**
	 * Attempt to reconnect to the store if the connection was severed, without recreating the store client.
	 * @return True on success, false on failure
	 */
	UE_API bool ReconnectToStore();

	UE_API void Disconnect();

	UE::Trace::FStoreClient* GetStoreClient() const { return StoreClient.Get(); }
	FCriticalSection& GetStoreClientCriticalSection() const { return StoreClientCriticalSection; }

	UE_API bool GetStoreAddressAndPort(uint32& OutStoreAddress, uint32& OutStorePort) const;
	UE_API FString GetStoreDir() const;

	const FString& GetLastStoreHost() const { return LastStoreHost; }
	uint32 GetLastStorePort() const { return LastStorePort; }

	const bool IsLocalHost() const { return bIsLocalHost; }
	const bool CanChangeStoreSettings() const { return bIsLocalHost && StoreClient.IsValid(); }

private:
	/** The client used to connect to the trace store. It is not thread safe! */
	TUniquePtr<UE::Trace::FStoreClient> StoreClient;

	/** CriticalSection for using the store client's API. */
	mutable FCriticalSection StoreClientCriticalSection;

	FString LastStoreHost;
	uint32 LastStorePort = 0;

	bool bIsLocalHost = false;
};

} // namespace UE::Trace

#undef UE_API
