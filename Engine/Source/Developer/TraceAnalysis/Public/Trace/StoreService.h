// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API TRACEANALYSIS_API

////////////////////////////////////////////////////////////////////////////////
namespace UE {
namespace Trace {

////////////////////////////////////////////////////////////////////////////////
class FStoreService
{
public:
	struct FDesc
	{
		const TCHAR*		StoreDir;
		int32				RecorderPort = 0; // 0:auto-assign, -1:off
		int32				ThreadCount  = 0; // <=0:logical CPU count
	};

							~FStoreService() = default;
	static UE_API FStoreService*	Create(const FDesc& Desc);
	UE_API void					operator delete (void* Addr);
	UE_API uint32					GetPort() const;
	UE_API uint32					GetRecorderPort() const;

private:
							FStoreService() = default;
							FStoreService(const FStoreService&) = delete;
							FStoreService(const FStoreService&&) = delete;
	void					operator = (const FStoreService&) = delete;
	void					operator = (const FStoreService&&) = delete;
};

} // namespace Trace
} // namespace UE

#undef UE_API
