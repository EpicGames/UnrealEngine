// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/AnsiString.h"
#include "Containers/UnrealString.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/Requires.h"

// TraceLog
#include "Trace/Detail/Protocol.h"
#include "Trace/Detail/Transport.h"

// TraceAnalysis
#include "Trace/Analyzer.h"

#include <type_traits>

#define UE_API TRACEANALYSIS_API

namespace UE::Trace
{

class IOutDataStream;
class FTraceWriter;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceWriterThreadInfo final
{
	friend class FTraceWriter;

public:
	FTraceWriterThreadInfo() = default;
	FTraceWriterThreadInfo(uint32 InThreadId, FAnsiStringView InName, uint32 InSystemId = 0, int32 InSortHint = 0)
		: Id(InThreadId)
		, Name(InName)
		, SystemId(InSystemId)
		, SortHint(InSortHint)
	{
	}

	[[nodiscard]] uint32 GetId() const { return Id; }
	[[nodiscard]] const FAnsiString& GetName() const { return Name; }
	[[nodiscard]] uint32 GetSystemId() const { return SystemId; }
	[[nodiscard]] int32 GetSortHint() const { return SortHint; }

	static constexpr uint32 InvalidThreadId = ~0u;

private:
	uint32 Id = InvalidThreadId;
	FAnsiString Name;
	uint32 SystemId = 0;
	int32 SortHint = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Flags for a trace event.
 * See UE::Trace::Protocol6::EEventFlags (Trace\Detail\Protocols\Protocol6.h in TraceLog module).
 * See UE::Trace::IAnalyzer::EEventFlags (Trace\Analyzer.h in TraceAnalysis module).
 */
enum class ETraceWriterEventFlags : uint8
{
	None            = 0,

	Important       = 1 << 0,
	MaybeHasAux     = 1 << 1,
	NoSync          = 1 << 2,
	Definition      = 1 << 3,

	ImportantNoSync = Important | NoSync,
};
ENUM_CLASS_FLAGS(ETraceWriterEventFlags);

////////////////////////////////////////////////////////////////////////////////////////////////////
/**
 * Field type.
 * See UE::Trace::Protocol0::EFieldType (Trace\Detail\Protocols\Protocol0.h in TraceLog module).
 * { Bool = 0, Int8, Int16, Int32, Int64, Uint8 = 0, Uint16, Uint32, Uint64, ..., Pointer = Uint64, ..., Array }
 * See also UE::Trace::IAnalyzer::FEventFieldInfo::EType
 * { None, Integer, Float, AnsiString, WideString, Reference8, Reference16, Reference32, Reference64 }
 */
enum class ETraceWriterFieldType : uint8
{
	ArrayFlag           = 0x80,
	ReferenceFlag       = 0x40, // can be combined with Uint8,Uint16,Uint32,Uint64 types
	DefinitionIdFlag    = 0x20, // can be combined with Uint8,Uint16,Uint32,Uint64 types
	FlagsMask           = 0xE0,
	IndexMask           = 0x1F,

	None                = 0,

	// Boolean type
	Bool                = 1, // 1 byte for a single bool value, variable size (1 bit / value) for a bool array

	// Unsigned integer types
	Uint8               = 2,
	Uint16              = 3,
	Uint32              = 4,
	//VarUInt32           = 5, // variable size, 1 to 5 bytes
	Uint64              = 6,
	//VarUInt64           = 7, // variable size, 1 to 9 bytes

	// Signed integer types
	Int8                = 8,
	Int16               = 9,
	Int32               = 10,
	//VarInt32            = 11, // variable size, 1 to 5 bytes
	Int64               = 12,
	//VarInt64            = 13, // variable size, 1 to 9 bytes

	// Floating-point number types
	Float32             = 14, // float
	Float64             = 15, // double

	// String types
	AnsiString          = 16,
	WideString          = 17,
	//Utf8String          = 18,

	Pointer             = 19, // void* (64 bits)

	Count               = 20,

	// Reference composed types
	Reference8          = ReferenceFlag | Uint8,
	Reference16         = ReferenceFlag | Uint16,
	Reference32         = ReferenceFlag | Uint32,
	Reference64         = ReferenceFlag | Uint64,

	// DefinitionId composed types
	DefinitionId8       = DefinitionIdFlag | Uint8,
	DefinitionId16      = DefinitionIdFlag | Uint16,
	DefinitionId32      = DefinitionIdFlag | Uint32,
	DefinitionId64      = DefinitionIdFlag | Uint64,
};
ENUM_CLASS_FLAGS(ETraceWriterFieldType);

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTraceWriterFieldTypeResolver
{
	template <typename NativeType>
	static constexpr ETraceWriterFieldType GetType()
	{
		using EType = ETraceWriterFieldType;

		     if constexpr (std::is_same_v<std::remove_const_t<NativeType>, bool  >) return EType::Bool;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, uint8 >) return EType::Uint8;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, uint16>) return EType::Uint16;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, uint32>) return EType::Uint32;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, uint64>) return EType::Uint64;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, int8  >) return EType::Int8;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, int16 >) return EType::Int16;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, int32 >) return EType::Int32;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, int64 >) return EType::Int64;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, float >) return EType::Float32;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, double>) return EType::Float64;

		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, bool[]  >) return EType::ArrayFlag | EType::Bool;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, uint8[] >) return EType::ArrayFlag | EType::Uint8;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, uint16[]>) return EType::ArrayFlag | EType::Uint16;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, uint32[]>) return EType::ArrayFlag | EType::Uint32;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, uint64[]>) return EType::ArrayFlag | EType::Uint64;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, int8[]  >) return EType::ArrayFlag | EType::Int8;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, int16[] >) return EType::ArrayFlag | EType::Int16;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, int32[] >) return EType::ArrayFlag | EType::Int32;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, int64[] >) return EType::ArrayFlag | EType::Int64;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, float[] >) return EType::ArrayFlag | EType::Float32;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, double[]>) return EType::ArrayFlag | EType::Float64;

		else if constexpr (std::is_same_v<NativeType, TConstArrayView<bool>  >) return EType::ArrayFlag | EType::Bool;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<uint8> >) return EType::ArrayFlag | EType::Uint8;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<uint16>>) return EType::ArrayFlag | EType::Uint16;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<uint32>>) return EType::ArrayFlag | EType::Uint32;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<uint64>>) return EType::ArrayFlag | EType::Uint64;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<int8>  >) return EType::ArrayFlag | EType::Int8;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<int16> >) return EType::ArrayFlag | EType::Int16;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<int32> >) return EType::ArrayFlag | EType::Int32;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<int64> >) return EType::ArrayFlag | EType::Int64;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<float> >) return EType::ArrayFlag | EType::Float32;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<double>>) return EType::ArrayFlag | EType::Float64;

		else if constexpr (std::is_same_v<NativeType, TArrayView<bool>  >) return EType::ArrayFlag | EType::Bool;
		else if constexpr (std::is_same_v<NativeType, TArrayView<uint8> >) return EType::ArrayFlag | EType::Uint8;
		else if constexpr (std::is_same_v<NativeType, TArrayView<uint16>>) return EType::ArrayFlag | EType::Uint16;
		else if constexpr (std::is_same_v<NativeType, TArrayView<uint32>>) return EType::ArrayFlag | EType::Uint32;
		else if constexpr (std::is_same_v<NativeType, TArrayView<uint64>>) return EType::ArrayFlag | EType::Uint64;
		else if constexpr (std::is_same_v<NativeType, TArrayView<int8>  >) return EType::ArrayFlag | EType::Int8;
		else if constexpr (std::is_same_v<NativeType, TArrayView<int16> >) return EType::ArrayFlag | EType::Int16;
		else if constexpr (std::is_same_v<NativeType, TArrayView<int32> >) return EType::ArrayFlag | EType::Int32;
		else if constexpr (std::is_same_v<NativeType, TArrayView<int64> >) return EType::ArrayFlag | EType::Int64;
		else if constexpr (std::is_same_v<NativeType, TArrayView<float> >) return EType::ArrayFlag | EType::Float32;
		else if constexpr (std::is_same_v<NativeType, TArrayView<double>>) return EType::ArrayFlag | EType::Float64;

		else if constexpr (std::is_same_v<NativeType, void*                       >) return EType::Pointer;
		else if constexpr (std::is_same_v<NativeType, const void*                 >) return EType::Pointer;
		else if constexpr (std::is_same_v<NativeType, void* []                    >) return EType::ArrayFlag | EType::Pointer;
		else if constexpr (std::is_same_v<NativeType, const void* []              >) return EType::ArrayFlag | EType::Pointer;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<void*>      >) return EType::ArrayFlag | EType::Pointer;
		else if constexpr (std::is_same_v<NativeType, TConstArrayView<const void*>>) return EType::ArrayFlag | EType::Pointer;
		else if constexpr (std::is_same_v<NativeType, TArrayView<void*>           >) return EType::ArrayFlag | EType::Pointer;
		else if constexpr (std::is_same_v<NativeType, TArrayView<const void*>     >) return EType::ArrayFlag | EType::Pointer;

		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, AnsiString>) return EType::AnsiString;
		else if constexpr (std::is_same_v<std::remove_const_t<NativeType>, WideString>) return EType::WideString;
		else static_assert(sizeof(NativeType) == 0, "Unknown NativeType");
	} //-V591
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceWriterEventFieldInfo final
{
	friend class FTraceWriter;
	friend class FTraceWriterEventDeclarationBuilder;

public:
	FTraceWriterEventFieldInfo() = delete;
	FTraceWriterEventFieldInfo(FAnsiStringView InFieldName, ETraceWriterFieldType InFieldType)
		: Name(FAnsiString(InFieldName))
		, Type(InFieldType)
	{
	}

	[[nodiscard]] const FAnsiString& GetName() const { return Name; }
	[[nodiscard]] ETraceWriterFieldType GetType() const { return Type; }
	[[nodiscard]] uint16 GetSize() const { return Size; }
	[[nodiscard]] uint16 GetRefUid() const { return RefUid; }

	[[nodiscard]] inline uint8 GetProtocolFieldFamily() const;
	[[nodiscard]] inline uint8 GetProtocolFieldType() const;

	bool IsArray() const { return EnumHasAnyFlags(Type, ETraceWriterFieldType::ArrayFlag); }

private:
	FAnsiString Name;
	ETraceWriterFieldType Type = ETraceWriterFieldType::None;
	uint16 Size = 0;
	uint16 RefUid = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceWriterEventInfo final
{
	friend class FTraceWriter;
	friend class FTraceWriterEventDeclarationBuilder;

private:
	FTraceWriterEventInfo() = delete;

	// Constructor only accessible from the FTraceWriter class
	FTraceWriterEventInfo(FAnsiStringView InLoggerName, FAnsiStringView InEventName, ETraceWriterEventFlags InEventFlags)
		: LoggerName(FAnsiString(InLoggerName))
		, Name(FAnsiString(InEventName))
		, Flags(InEventFlags)
	{
	}

public:
	[[nodiscard]] uint32 GetId() const { return Id; }
	[[nodiscard]] const FAnsiString& GetLoggerName() const { return LoggerName; }
	[[nodiscard]] const FAnsiString& GetName() const { return Name; }
	[[nodiscard]] ETraceWriterEventFlags GetFlags() const { return Flags; }
	[[nodiscard]] uint32 GetNumFields() const { return static_cast<uint32>(Fields.Num()); }
	[[nodiscard]] const FTraceWriterEventFieldInfo& GetField(uint32 InFieldIndex) const { return Fields[InFieldIndex]; }
	[[nodiscard]] const TArray<FTraceWriterEventFieldInfo>& GetFields() const { return Fields; }
	[[nodiscard]] UE_API uint32 GetFieldIndex(FAnsiStringView InFieldName) const;

	bool IsNoSync() const { return EnumHasAnyFlags(Flags, ETraceWriterEventFlags::NoSync); }
	bool IsSync() const { return !EnumHasAnyFlags(Flags, ETraceWriterEventFlags::NoSync); }
	bool IsImportant() const { return EnumHasAnyFlags(Flags, ETraceWriterEventFlags::Important); }
	bool MaybeHasAux() const { return EnumHasAnyFlags(Flags, ETraceWriterEventFlags::MaybeHasAux); }
	bool IsDefinition() const { return EnumHasAnyFlags(Flags, ETraceWriterEventFlags::Definition); }

private:
	uint32 Id = 0;
	FAnsiString LoggerName;
	FAnsiString Name;
	ETraceWriterEventFlags Flags = ETraceWriterEventFlags::None;
	TArray<FTraceWriterEventFieldInfo> Fields;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceWriterEventDeclarationBuilder final
{
	friend class FTraceWriter;

private:
	FTraceWriterEventDeclarationBuilder() = delete;

	// Constructor only accessible from the FTraceWriter class
	FTraceWriterEventDeclarationBuilder(FTraceWriter& InWriter, FTraceWriterEventInfo& InEventInfo)
		: Writer(InWriter)
		, EventInfo(InEventInfo)
	{
	}

	~FTraceWriterEventDeclarationBuilder();

public:
	[[nodiscard]] FTraceWriterEventInfo& GetEventInfo() const
	{
		return EventInfo;
	}

	/**
	 * Adds a new regular field to the current event declaration.
	 *
	 * @param InFieldName - The name of the field.
	 * @param InFieldType - The type of the field.
	 * @returns - The event declaration builder object.
	 */
	UE_API FTraceWriterEventDeclarationBuilder& Field(FAnsiStringView InFieldName, ETraceWriterFieldType InFieldType);

	/**
	 * Adds a new Reference field to the current event declaration.
	 *
	 * @param InFieldName - The name of the field.
	 * @param InFieldType - The type of the field. Can only be Reference8, Reference16, Reference32 or Reference64.
	 * @param InRefUid - The reference unique id.
	 * @returns - The event declaration builder object.
	 */
	UE_API FTraceWriterEventDeclarationBuilder& ReferenceField(FAnsiStringView InFieldName, ETraceWriterFieldType InFieldType, uint16 InRefUid);

	/**
	 * Adds a new DefinitionId field to the current event declaration.
	 * @param InFieldType - The type of the field. Can only be DefinitionId8, DefinitionId16, DefinitionId32 or DefinitionI64.
	 * @returns - The event declaration builder object.
	 */
	UE_API FTraceWriterEventDeclarationBuilder& DefinitionIdField(ETraceWriterFieldType InFieldType);

	/**
	 * Signals the event declaration is complete.
	 *
	 * @returns the event id of the newly declared event
	 */
	[[nodiscard]] UE_API uint32 End();

private:
	FTraceWriter& Writer;
	FTraceWriterEventInfo& EventInfo;
	bool bIsCompleted = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceWriterFieldData
{
public:
	FTraceWriterFieldData() = default;
	~FTraceWriterFieldData();

	const void* GetData() const
	{
		return Size <= MaxInlineDataSize ? &Data : Buffer;
	}
	uint32 GetSize() const
	{
		return Size;
	}
	UE_API void SetData(const void* InData, uint32 InDataSize);

private:
	static constexpr uint32 MaxInlineDataSize = 12; // Data + Data2
	union
	{
		void* Buffer;
		uint64 Data = 0;
	};
	uint32 Data2 = 0;
	uint32 Size = 0;
};
static_assert(sizeof(FTraceWriterFieldData) == 16, "");

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceWriterEventBuilder final
{
	friend class FTraceWriter;

private:
	FTraceWriterEventBuilder(FTraceWriter& InWriter, const FTraceWriterEventInfo& InEventInfo)
		: Writer(InWriter)
		, EventInfo(InEventInfo)
	{
		if (InEventInfo.GetNumFields() > 0)
		{
			Fields.AddDefaulted(InEventInfo.GetNumFields());
		}
	}

	~FTraceWriterEventBuilder();

public:
	[[nodiscard]] const FTraceWriterEventInfo& GetEventInfo() const
	{
		return EventInfo;
	}

	//////////////////////////////////////////////////
	// Array types (T[])
	// Value is passed as Data + DataSize [bytes], as TConstArrayView<T> or as TArrayView<T>.

	UE_API FTraceWriterEventBuilder& Field(uint32 Index, const void* Data, uint32 DataSize);

	FTraceWriterEventBuilder& Field(FAnsiStringView Name, const void* Data, uint32 DataSize)
	{
		return Field(EventInfo.GetFieldIndex(Name), Data, DataSize);
	}

	template <typename T>
	FTraceWriterEventBuilder& Field(uint32 Index, TConstArrayView<T> ArrayView)
	{
		return Field(Index, ArrayView.GetData(), ArrayView.Num() * sizeof(T));
	}

	template <typename T>
	FTraceWriterEventBuilder& Field(uint32 Index, TArrayView<T> ArrayView)
	{
		return Field(Index, ArrayView.GetData(), ArrayView.Num() * sizeof(T));
	}

	//////////////////////////////////////////////////
	// Integral and floating point types
	// (bool, uint8, uint16,..., float, double)

	template <
		typename T
		UE_REQUIRES(
			std::is_integral_v<T> || std::is_floating_point_v<T>
		)
	>
	FTraceWriterEventBuilder& Field(uint32 Index, T Value)
	{
		constexpr ETraceWriterFieldType FieldType = FTraceWriterFieldTypeResolver::GetType<T>();
		if (CheckValidPodField(Index, FieldType, sizeof(T)))
		{
			Fields[Index].SetData(&Value, sizeof(T));
		}
		return *this;
	}

	//////////////////////////////////////////////////
	// Pointer type (void*)

	FTraceWriterEventBuilder& Field(uint32 Index, const void* Value)
	{
		if (CheckValidPodField(Index, ETraceWriterFieldType::Pointer, sizeof(void*)))
		{
			Fields[Index].SetData(&Value, sizeof(void*));
		}
		return *this;
	}

	FTraceWriterEventBuilder& Field(FAnsiStringView Name, const void* Value)
	{
		return Field(EventInfo.GetFieldIndex(Name), Value);
	}

	FTraceWriterEventBuilder& Field(uint32 Index, void* Value)
	{
		if (CheckValidPodField(Index, ETraceWriterFieldType::Pointer, sizeof(void*)))
		{
			Fields[Index].SetData(&Value, sizeof(void*));
		}
		return *this;
	}

	FTraceWriterEventBuilder& Field(FAnsiStringView Name, void* Value)
	{
		return Field(EventInfo.GetFieldIndex(Name), Value);
	}

	//////////////////////////////////////////////////
	// Reference types

	FTraceWriterEventBuilder& Field(uint32 Index, UE::Trace::FEventRef8 Value)
	{
		if (CheckValidPodField(Index, ETraceWriterFieldType::Reference8, 1))
		{
			Fields[Index].SetData(&Value.Id, 1);
		}
		return *this;
	}

	FTraceWriterEventBuilder& Field(uint32 Index, UE::Trace::FEventRef16 Value)
	{
		if (CheckValidPodField(Index, ETraceWriterFieldType::Reference16, 2))
		{
			Fields[Index].SetData(&Value.Id, 2);
		}
		return *this;
	}

	FTraceWriterEventBuilder& Field(uint32 Index, UE::Trace::FEventRef32 Value)
	{
		if (CheckValidPodField(Index, ETraceWriterFieldType::Reference32, 4))
		{
			Fields[Index].SetData(&Value.Id, 4);
		}
		return *this;
	}

	FTraceWriterEventBuilder& Field(uint32 Index, UE::Trace::FEventRef64 Value)
	{
		if (CheckValidPodField(Index, ETraceWriterFieldType::Reference64, 8))
		{
			Fields[Index].SetData(&Value.Id, 8);
		}
		return *this;
	}

	//////////////////////////////////////////////////
	// AnsiString

	/** Construct field from an FAnsiStringView. */
	template <typename T = FAnsiStringView>
	FTraceWriterEventBuilder& Field(uint32 Index, FAnsiStringView Value UE_LIFETIMEBOUND)
	{
		if (CheckValidExactFieldType(Index, ETraceWriterFieldType::AnsiString))
		{
			SetField(Index, (const void*)Value.GetData(), (uint32)Value.NumBytes());
		}
		return *this;
	}

	/** Construct field from the null-terminated string pointed to by InData. */
	template <typename T = const ANSICHAR*>
	FTraceWriterEventBuilder& Field(uint32 Index, const ANSICHAR* InData UE_LIFETIMEBOUND)
	{
		return Field(Index, FAnsiStringView(InData));
	}

	/** Construct field from an FAnsiString. */
	template <typename T = const FAnsiString&>
	FTraceWriterEventBuilder& Field(uint32 Index, const FAnsiString& Value UE_LIFETIMEBOUND)
	{
		return Field(Index, FAnsiStringView(Value));
	}

	//////////////////////////////////////////////////
	// WideString

	/** Construct field from an FStringView. */
	template <typename T = FStringView>
	FTraceWriterEventBuilder& Field(uint32 Index, const FStringView Value UE_LIFETIMEBOUND)
	{
		if (CheckValidExactFieldType(Index, ETraceWriterFieldType::WideString))
		{
			SetField(Index, Value.GetData(), (uint32)Value.NumBytes());
		}
		return *this;
	}

	/** Construct field from the null-terminated string pointed to by InData. */
	template <typename T = const TCHAR*>
	FTraceWriterEventBuilder& Field(uint32 Index, const TCHAR* Value UE_LIFETIMEBOUND)
	{
		return Field(Index, FStringView(Value));
	}

	/** Construct field from an FString. */
	template <typename T = const FString&>
	FTraceWriterEventBuilder& Field(uint32 Index, const FString& Value UE_LIFETIMEBOUND)
	{
		return Field(Index, FStringView(Value));
	}

	//////////////////////////////////////////////////
	// Identify the field by name.

	template <typename T>
	FTraceWriterEventBuilder& Field(FAnsiStringView Name, T Value)
	{
		return Field(EventInfo.GetFieldIndex(Name), Value);
	}

	//////////////////////////////////////////////////

	UE_API void End();

private:
	void SetErrorInvalidFieldIndex(uint32 Index);
	void SetErrorInvalidFieldType(uint32 Index);
	void SetErrorInvalidFieldData(uint32 Index);

	UE_API bool CheckValidPodField(uint32 Index, ETraceWriterFieldType FieldType, uint32 Size);
	UE_API bool CheckValidExactFieldType(uint32 Index, ETraceWriterFieldType FieldType);
	UE_API void SetField(uint32 Index, const void* Data, uint32 DataSize);

private:
	FTraceWriter& Writer;
	const FTraceWriterEventInfo& EventInfo;
	TArray<FTraceWriterFieldData> Fields;
	bool bIsCompleted = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETraceWriterBeginOptions : uint32
{
	None = 0,

	DeclareDefaultEvents = 1 << 1,
	WriteNewTraceEvent = 1 << 2,

	Default = DeclareDefaultEvents | WriteNewTraceEvent
};
ENUM_CLASS_FLAGS(ETraceWriterBeginOptions);

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceWriter
{
	friend class FTraceWriterEventInfo;
	friend class FTraceWriterEventDeclarationBuilder;
	friend class FTraceWriterEventBuilder;

private:
	struct FTraceGuid
	{
		uint32 Bits[4];
	};

public:
	UE_API FTraceWriter(IOutDataStream& InStream);
	UE_API ~FTraceWriter();

	FTraceWriter(const FTraceWriter&) = delete;
	FTraceWriter& operator=(const FTraceWriter&) = delete;

	/**
	 * Returns the version of the transport protocol used by this trace writer.
	 */
	UE_API UE::Trace::ETransport GetTransportProtocolVersion() const;

	/**
	 * Returns the version of the trace protocol used by this trace writer.
	 * Currently this is Protocol 7.
	 *
	 * \details
	 * See UE::Trace::Protocol7::EProtocol::Id (Trace/Detail/Protocols/Protocol7.h).
	 */
	UE_API uint8 GetTraceProtocolVersion() const;

	/**
	 * Begins the trace stream.
	 * Calls DeclareDefaultEvents() and WriteNewTraceEvent() depending on Options flags.
	 *
	 * @param Options - Various toggle options to allow custom control of Begin() behavior
	 */
	UE_API void Begin(ETraceWriterBeginOptions Options = ETraceWriterBeginOptions::Default);

	/**
	 * Ends the trace stream.
	 *
	 * \details
	 * Calls Close() for the stream.
	 */
	UE_API void End();

	/**
	 * Declares the default trace events.
	 *
	 * \details
	 * Declares: $Trace.* events
	 */
	UE_API void DeclareDefaultEvents();

	/**
	 * Writes the $Trace.NewTrace event.
	 * Changes the current thread to be the "Importants" thread.
	 *
	 * \details
	 * Writes: $Trace.NewTrace
	 */
	UE_API void WriteNewTraceEvent();

	//////////////////////////////////////////////////
	// Threads

	/**
	 * Registers a new trace thread.
	 * If a thread with same name is already registered, this will just return the registered thread id.
	 * If successfully registering a new thread and bShouldWriteThreadInfo is true, then this function
	 * will call WriteThreadInfo() automatically.
	 *
	 * @param ThreadName - The thread name
	 * @param SystemId - The system thread id
	 * @param SortHint - A hint for sorting priority
	 * @param bShouldWriteThreadInfo - Wherever should write the $Trace.ThreadInfo or not
	 *
	 * @returns the thread id for the new trace thread
	 */
	UE_API uint32 RegisterThread(FAnsiStringView ThreadName, uint32 SystemId = 0, int32 SortHint = 0, bool bShouldWriteThreadInfo = true);

	/**
	 * Registers a trace thread with a known id.
	 * If a thread with the specified id is already registered, this will fail.
	 * If successfully registering a new thread and bShouldWriteThreadInfo is true, then this function
	 * will call WriteThreadInfo() automatically.
	 *
	 * @param ThreadId - The thread id
	 * @param ThreadName - The thread name
	 * @param SystemId - The system thread id
	 * @param SortHint - A hint for sorting priority
	 * @param bShouldWriteThreadInfo - Wherever should write the $Trace.ThreadInfo or not
	 */
	UE_API void RegisterCustomThread(uint32 ThreadId, FAnsiStringView ThreadName, uint32 SystemId = 0, int32 SortHint = 0, bool bShouldWriteThreadInfo = true);

	/**
	 * Gets the info for a registered thread identified by Id.
	 *
	 * @param ThreadId - The thread id
	 * @returns a pointer to the thread info or nullptr if the ThreadId is not valid (thread not registered).
	 */
	const FTraceWriterThreadInfo* GetThreadInfo(uint32 ThreadId)
	{
		return ThreadId < uint32(Threads.Num()) && Threads[ThreadId].Id != FTraceWriterThreadInfo::InvalidThreadId ? &Threads[ThreadId] : nullptr;
	}

	/**
	 * Writes the $Trace.ThreadInfo event for a registered thread, identified by id.
	 * Changes the current thread to be the "Importants" thread.
	 *
	 * \details
	 * Writes: $Trace.ThreadInfo
	 *
	 * @param ThreadId - The thread id
	 */
	UE_API void WriteThreadInfo(uint32 ThreadId);

	/**
	 * Writes the $Trace.ThreadInfo event.
	 * Changes the current thread to be the "Importants" thread.
	 *
	 * \details
	 * Writes: $Trace.ThreadInfo
	 *
	 * @param ThreadId - The thread id
	 * @param ThreadName - The thread name
	 * @param SystemId - The system thread id
	 * @param SortHint - A hint for sorting priority
	 */
	UE_API void WriteThreadInfo(uint32 ThreadId, FAnsiStringView ThreadName, uint32 SystemId = 0, int32 SortHint = 0);

	/**
	 * Sets the current thread id.
	 *
	 * @param ThreadId - The thread id.
	 */
	UE_API void SetCurrentThread(uint32 ThreadId);

	/**
	 * Sets the current thread to be the "Events" thread.
	 */
	void SetCurrentThreadEvents() { SetCurrentThread(static_cast<uint32>(ETransportTid::Events)); }

	/**
	 * Sets the current thread to be the "Importants" thread.
	 */
	void SetCurrentThreadImportants() { SetCurrentThread(static_cast<uint32>(ETransportTid::Importants)); }

	/**
	 * Gets the current thread id.
	 *
	 * @returns the current thread id
	 */
	uint32 GetCurrentThread() const { return CurrentThreadId; }

	/**
	 * Writes a $Trace.ThreadTiming event for the current thread.
	 *
	 * @param BaseTimestamp - The base timestamp [ticks]
	 */
	UE_API void WriteThreadTimingEvent(uint64 BaseTimestamp);

	/**
	 * Writes a $Trace.ThreadGroupBegin event.
	 * Changes the current thread to be the "Importants" thread.
	 *
	 * \details
	 * Writes: $Trace.ThreadGroupBegin
	 *
	 * @param GroupName - The group name
	 */
	UE_API void WriteThreadGroupBeginEvent(FAnsiStringView GroupName);

	/**
	 * Writes a $Trace.ThreadGroupEnd event.
	 * Changes the current thread to be the "Importants" thread.
	 *
	 * \details
	 * Writes: $Trace.ThreadGroupEnd
	 */
	UE_API void WriteThreadGroupEndEvent();

	//////////////////////////////////////////////////

	/**
	 * Gets the info for an event identified by Id.
	 *
	 * @param EventId - The event id, one returned by DeclareEvent().[...].End() or similar method.
	 * @returns a pointer to the event info or nullptr if the EventId is not valid.
	 */
	const FTraceWriterEventInfo* GetEventInfo(uint32 EventId) const
	{
		return EventId < uint32(EventInfos.Num()) ? EventInfos[EventId] : nullptr;
	}

	/**
	 * Gets the info for an event identified by logger and event names.
	 *
	 * @param LoggerName - The logger name
	 * @param Name - The event name
	 * @returns a pointer to the event info or nullptr if the event is not found.
	 */
	UE_API const FTraceWriterEventInfo* FindEvent(FAnsiStringView LoggerName, FAnsiStringView EventName) const;

	/**
	 * Declares a new trace event.
	 * The End() call for the returned builder will change the current thread to be the "Events" thread.
	 *
	 * \details
	 * Declares: the new trace event
	 * Writes: $Trace.NewEvent
	 *
	 * @param LoggerName - The logger name
	 * @param Name - The event name
	 * @param Flags - The event flags (Important, NoSync, etc.)
	 * @returns an event declaration builder object (FTraceWriterEventDeclarationBuilder);
	 *          The builder object is used to further declare the event fields.
	 *          User needs to call the builder's End() to complete the event declaration
	 *          and to get the event id for the newly declared event.
	 */
	[[nodiscard]] UE_API FTraceWriterEventDeclarationBuilder& DeclareEvent(FAnsiStringView LoggerName, FAnsiStringView Name, ETraceWriterEventFlags Flags = ETraceWriterEventFlags::None);

	/**
	 * Declares the Diagnostics.Session2 event.
	 *
	 * \details
	 * Declares: Diagnostics.Session2 event
	 * Writes: $Trace.NewEvent
	 */
	[[nodiscard]] UE_API uint32 DeclareDiagnosticsSession2Event();

	/**
	 * Writes the specified event, on the current thread.
	 *
	 * @param EventId - The event id, one returned by DeclareEvent().[...].End()
	 * @returns an event builder object (FTraceWriterEventBuilder);
	 *          The builder object is used to further set values for each event field.
	 *          User needs to call the builder's End() to complete the event.
	 */
	[[nodiscard]] UE_API FTraceWriterEventBuilder& WriteEvent(uint32 EventId);

	//////////////////////////////////////////////////
	// Scoped Events

	/**
	 * Writes the EnterScope event, on the current thread.
	 */
	UE_API void WriteEnterScopeEvent();
	/**
	 * Writes the LeaveScope event, on the current thread.
	 */
	UE_API void WriteLeaveScopeEvent();

	/**
	 * Writes the EnterScope_TB event, on the current thread, using the specified timestamp.
	 *
	 * @param Timestamp - The current session time (i.e. one returned by GetTime()).
	 */
	UE_API void WriteStampedEnterScopeEvent(uint64 Timestamp);

	/**
	 * Writes the LeaveScope_TB event, on the current thread, using the specified timestamp.
	 *
	 * @param Timestamp - The current session time (i.e. one returned by GetTime()).
	 */
	UE_API void WriteStampedLeaveScopeEvent(uint64 Timestamp);

	//////////////////////////////////////////////////
	// Event Flags

	[[nodiscard]] static ETraceWriterEventFlags ConvertEventFlags(UE::Trace::IAnalyzer::EEventFlags Flags)
	{
		static_assert((uint8)UE::Trace::IAnalyzer::EEventFlags::None == (uint8)ETraceWriterEventFlags::None, "");
		static_assert((uint8)UE::Trace::IAnalyzer::EEventFlags::Important == (uint8)ETraceWriterEventFlags::Important, "");
		static_assert((uint8)UE::Trace::IAnalyzer::EEventFlags::MaybeHasAux == (uint8)ETraceWriterEventFlags::MaybeHasAux, "");
		static_assert((uint8)UE::Trace::IAnalyzer::EEventFlags::Definition == (uint8)ETraceWriterEventFlags::Definition, "");
		return static_cast<ETraceWriterEventFlags>(Flags);
	}

	[[nodiscard]] static uint8 GetProtocolEventFlags(ETraceWriterEventFlags Flags)
	{
		using namespace UE::Trace::Protocol7;
		static_assert((uint8)ETraceWriterEventFlags::None == (uint8)EEventFlags::None, "");
		static_assert((uint8)ETraceWriterEventFlags::Important == (uint8)EEventFlags::Important, "");
		static_assert((uint8)ETraceWriterEventFlags::MaybeHasAux == (uint8)EEventFlags::MaybeHasAux, "");
		static_assert((uint8)ETraceWriterEventFlags::Definition == (uint8)EEventFlags::Definition, "");
		return static_cast<uint8>(Flags);
	}

	//////////////////////////////////////////////////
	// Field Types

	UE_API static uint16 GetByteSizeForFieldType(ETraceWriterFieldType Type);

	[[nodiscard]] static bool IsArrayFieldType(ETraceWriterFieldType Type)
	{
		return EnumHasAnyFlags(Type, ETraceWriterFieldType::ArrayFlag);
	}
	[[nodiscard]] static bool IsSignedFieldType(ETraceWriterFieldType Type)
	{
		return (uint32)Type >= (uint32)ETraceWriterFieldType::Int8 && (uint32)Type <= (uint32)ETraceWriterFieldType::Float64;
	}
	[[nodiscard]] static bool IsReferenceFieldType(ETraceWriterFieldType Type)
	{
		return EnumHasAnyFlags(Type, ETraceWriterFieldType::ReferenceFlag);
	}
	[[nodiscard]] static bool IsDefinitionIdFieldType(ETraceWriterFieldType Type)
	{
		return EnumHasAnyFlags(Type, ETraceWriterFieldType::DefinitionIdFlag);
	}

	[[nodiscard]] UE_API static uint8 GetProtocolFieldFamily(ETraceWriterFieldType Type);
	[[nodiscard]] UE_API static uint8 GetProtocolFieldType(ETraceWriterFieldType Type);

	[[nodiscard]] UE_API static ETraceWriterFieldType ConvertFieldType(const UE::Trace::IAnalyzer::FEventFieldInfo& FieldInfo);

	//////////////////////////////////////////////////
	// Time

	/**
	 * Sets the custom clock for the session time.
	 * Warning: It is only valid to set these before writer's Begin().
	 *
	 * @param InTimeFrequency - The time frequency; the number of [ticks] in a second.
	 * @param InTimeGetter - The custom getter for the session time; when called it is expected to return the current session time, in [ticks].
	 * @param InStartTime - The start session timestamp, in [ticks].
	 * @param InStartTimeSinceEpoch - The start session time since Unix Epoch (January 1st of 1970), in [seconds]. Should match InStartTime.
	 */
	void SetCustomClock(uint64 InTimeFrequency, TFunction<uint64()> InTimeGetter, uint64 InStartTime, double InStartTimeSinceEpoch)
	{
		CycleFrequency = InTimeFrequency;
		TimeGetter = InTimeGetter;
		StartCycle = InStartTime;
		StartTimeSinceEpoch = InStartTimeSinceEpoch;
		bUseCustomClock = true;
	}

	/**
	 * Returns the frequency of the session time, specified as number of [ticks] per second.
	 *
	 * @returns the frequency of the session time.
	 */
	[[nodiscard]] uint64 GetTimeFrequency() const
	{
		return CycleFrequency;
	}

	/**
	 * Gets the start session time, in [ticks].
	 *
	 * @returns the start session time.
	 */
	[[nodiscard]] uint64 GetStartTime() const
	{
		return StartCycle;
	}

	/**
	 * Gets the start session time since the Unix Epoch (January 1st of 1970), in [seconds].
	 *
	 * @returns the start session time since the Unix Epoch.
	 */
	[[nodiscard]] double GetStartTimeSinceEpoch() const
	{
		return StartTimeSinceEpoch;
	}

	/**
	 * Gets the current session time, in [ticks].
	 *
	 * @returns the current session time.
	 */
	[[nodiscard]] UE_API uint64 GetTime() const
	{
		return TimeGetter();
	}

	/**
	 * Gets the current session time since the Unix Epoch (January 1st of 1970), in [seconds].
	 *
	 * @returns the current session time since the Unix Epoch.
	 */
	[[nodiscard]] double GetTimeSinceEpoch() const
	{
		const uint64 RelativeTime = GetTime() - StartCycle;
		return StartTimeSinceEpoch + double(RelativeTime) / double(CycleFrequency);
	}

	//////////////////////////////////////////////////

	/**
	 * Returns the last error message from the stack. Returns an empty string if there was no error.
	 * Continuing to call GetLastError() will extract the previous emitted error until the stack is empty.
	 */
	[[nodiscard]] UE_API const FString& GetLastError() const;

	/**
	 * Clears the stack of error messages, if any.
	 */
	void ClearErrors() { Errors.Reset(); }

	/**
	 * Returns wherever safe mode is enabled or not.
	 */
	bool IsSafeModeEnabled() const { return true; }

private:
	void Reset();
	static void GenerateGuid(FTraceGuid* OutGuid);
	uint32 GetOrDeclareUnknownEventId();

	void WriteEventDeclaration(const FTraceWriterEventDeclarationBuilder& Builder);
	void WriteEventDeclaration(const FTraceWriterEventInfo& EventInfo);
	void WriteEvent(const FTraceWriterEventBuilder& Builder);
	void WriteEvent(const FTraceWriterEventBuilder& Builder, const FTraceWriterEventInfo& EventInfo);
	void WriteAuxData(uint32 Index, const void* Data, uint32 Size, bool bIsImportant);
	void WriteAuxDataSegment(uint32 Index, const void* SegmentData, uint32 SegmentSize, bool bIsImportant);

	void WriteScopeEventPrivate(uint8 Uid);
	void WriteStampedScopeEventPrivate(uint8 Uid, uint64 Timestamp);

	uint8* GetBuffer(uint32 RequiredDataSize);
	void AdvanceBuffer(uint32 DataSize);
	void BeginEvent();
	void EndEvent();
	void WritePacket();

	void InitDefaultClock();
	void SetError(FString&& InError);

private:
	IOutDataStream& Stream;

	FTraceGuid TraceGuid;
	FTraceGuid SessionGuid;

	uint64 CycleFrequency = 0; // [ticks] per second
	TFunction<uint64()> TimeGetter; // returns session time, in [ticks]
	uint64 StartCycle = 0; // [ticks]
	double StartTimeSinceEpoch = 0.0; // [seconds] since epoch
	bool bUseCustomClock = false;

	uint32 NextSerial = 0;

	uint32 UnknownEventId = 0;
	uint32 NewTraceEventId = 0;
	uint32 ThreadTimingEventId = 0;
	uint32 ThreadInfoEventId = 0;
	uint32 ThreadGroupBeginEventId = 0;
	uint32 ThreadGroupEndEventId = 0;

	TArray<FTraceWriterThreadInfo> Threads;

	TArray<FTraceWriterEventInfo*> EventInfos;
	FTraceWriterEventDeclarationBuilder* PendingEventDeclaration = nullptr;
	FTraceWriterEventBuilder* PendingEvent = nullptr;

	uint32 CurrentThreadId = ETransportTid::Events;
	uint8* EventBuffer = nullptr;
	uint32 EventBufferSize = 0;
	static constexpr uint32 MaxPacketHeaderSize = 8; // >= sizeof(FTidPacketEncoded), >= sizeof(FTidPacket)
	static constexpr uint32 EncodingOverhead = 64; // LZ4_COMPRESSBOUND
	static constexpr uint32 MaxPacketBufferSize = 64 << 10;
	static constexpr uint32 MaxEncodedBufferSize = MaxPacketBufferSize - EncodingOverhead - MaxPacketHeaderSize;
	static constexpr uint32 MaxDecodedBufferSize = MaxEncodedBufferSize;
	static constexpr uint32 MinEncodedBufferSize = 384;
	static constexpr uint32 FlushPacketThreshold = 32 << 10;
	uint32 EventBufferDataSize = 0;
	uint32 CompletedEventsDataSize = 0;
	uint8* PacketBuffer = nullptr;

	mutable TArray<FString> Errors;
	mutable FString LastError;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

uint8 FTraceWriterEventFieldInfo::GetProtocolFieldFamily() const
{
	return FTraceWriter::GetProtocolFieldFamily(Type);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint8 FTraceWriterEventFieldInfo::GetProtocolFieldType() const
{
	return FTraceWriter::GetProtocolFieldType(Type);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#define UE_ADHOC_TRACE_BEGIN_DECLARE_EVENT(Writer, LoggerName, EventName, Flags) \
	Writer.DeclareEvent(ANSITEXTVIEW(LoggerName), ANSITEXTVIEW(EventName), ETraceWriterEventFlags:: Flags)

#define UE_ADHOC_TRACE_DECLARE_FIELD(Type, FieldName) \
	.Field(ANSITEXTVIEW(FieldName), FTraceWriterFieldTypeResolver::GetType<Type>())

#define UE_ADHOC_TRACE_END_DECLARE_EVENT \
	.End();

#define __UE_ADHOC_TRACE_NARG(...) __UE_ADHOC_TRACE_NARG_(__VA_ARGS__, __UE_ADHOC_TRACE_RSEQ_N())
#define __UE_ADHOC_TRACE_NARG_(...) __UE_ADHOC_TRACE_ARG_N(__VA_ARGS__)
#define __UE_ADHOC_TRACE_ARG_N(_1, _2, _3, _4, _5, _6, _7, _8, N, ...) N
#define __UE_ADHOC_TRACE_RSEQ_N() 8, 7, 6, 5, 4, 3, 2, 1, 0

#define __UE_ADHOC_TRACE_CONCATENATE(a, b) __UE_ADHOC_TRACE_CONCATENATE2(a, b)
#define __UE_ADHOC_TRACE_CONCATENATE2(a, b) a##b

#define __UE_ADHOC_TRACE_LOG_ARGS_1(Arg0)											.Field(0, Arg0)
#define __UE_ADHOC_TRACE_LOG_ARGS_2(Arg0, Arg1)										.Field(0, Arg0).Field(1, Arg1)
#define __UE_ADHOC_TRACE_LOG_ARGS_3(Arg0, Arg1, Arg2)								.Field(0, Arg0).Field(1, Arg1).Field(2, Arg2)
#define __UE_ADHOC_TRACE_LOG_ARGS_4(Arg0, Arg1, Arg2, Arg3)							.Field(0, Arg0).Field(1, Arg1).Field(2, Arg2).Field(3, Arg3)
#define __UE_ADHOC_TRACE_LOG_ARGS_5(Arg0, Arg1, Arg2, Arg3, Arg4)					.Field(0, Arg0).Field(1, Arg1).Field(2, Arg2).Field(3, Arg3).Field(4, Arg4)
#define __UE_ADHOC_TRACE_LOG_ARGS_6(Arg0, Arg1, Arg2, Arg3, Arg4, Arg5)				.Field(0, Arg0).Field(1, Arg1).Field(2, Arg2).Field(3, Arg3).Field(4, Arg4).Field(5, Arg5)
#define __UE_ADHOC_TRACE_LOG_ARGS_7(Arg0, Arg1, Arg2, Arg3, Arg4, Arg5, Arg6)		.Field(0, Arg0).Field(1, Arg1).Field(2, Arg2).Field(3, Arg3).Field(4, Arg4).Field(5, Arg5).Field(6, Arg6)
#define __UE_ADHOC_TRACE_LOG_ARGS_8(Arg0, Arg1, Arg2, Arg3, Arg4, Arg5, Arg6, Arg7)	.Field(0, Arg0).Field(1, Arg1).Field(2, Arg2).Field(3, Arg3).Field(4, Arg4).Field(5, Arg5).Field(6, Arg6).Field(7, Arg7)

#define UE_ADHOC_TRACE_LOG(Writer, Uid, ...) \
	Writer.WriteEvent(Uid) \
		__UE_ADHOC_TRACE_CONCATENATE(__UE_ADHOC_TRACE_LOG_ARGS_,__UE_ADHOC_TRACE_NARG(__VA_ARGS__))(__VA_ARGS__) \
		.End();

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Trace

#undef UE_API
