// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// Enables debug API in UE::TRACE::IAnalyzer class (and sub-types).
// Enabling it should not add any impact on analysis' performance or behaviour.
#if !defined(UE_TRACE_ANALYSIS_DEBUG_API)
	#define UE_TRACE_ANALYSIS_DEBUG_API 0
#endif

// Enables debug functionality and logging in TraceAnalysis code.
// Enabling it can add significant impact on analysis' performance and it could alter behaviour.
#if !defined(UE_TRACE_ANALYSIS_DEBUG)
	#define UE_TRACE_ANALYSIS_DEBUG 0
#endif

// Default low level logging API used when UE_TRACE_ANALYSIS_DEBUG is enabled.
// For now, only ANSICHAR* strings are supported for logging.
#if !defined(UE_TRACE_ANALYSIS_DEBUG_LOG)
#if UE_TRACE_ANALYSIS_DEBUG

// Default implementation of UE_TRACE_ANALYSIS_DEBUG_LOG API:
// 1: only printf
// 2: printf with TAnsiStringBuilder
// 3: GLog
// 4: FPlatformMisc::LowLevelOutputDebugStringf
#if !defined(UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL)
	#define UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL 1
#endif

#if UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL == 1 // only printf

	#define UE_TRACE_ANALYSIS_DEBUG_LOG(format, ...)        printf(format "\n", ##__VA_ARGS__)
	#define UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder()
	#define UE_TRACE_ANALYSIS_DEBUG_Append(s)               printf("%s", s)
	#define UE_TRACE_ANALYSIS_DEBUG_Appendf(format, ...)    printf(format, ##__VA_ARGS__)
	#define UE_TRACE_ANALYSIS_DEBUG_AppendChar(c)           printf("%c", c)
	#define UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder()      printf("\n")
	#define UE_TRACE_ANALYSIS_DEBUG_ResetStringBuilder()

#elif UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL == 2 // printf with TAnsiStringBuilder

	#define UE_TRACE_ANALYSIS_DEBUG_LOG(format, ...)        printf(format "\n", ##__VA_ARGS__)
	#define UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder()    TAnsiStringBuilder<1024> StringBuilder;
	#define UE_TRACE_ANALYSIS_DEBUG_Append                  StringBuilder.Append
	#define UE_TRACE_ANALYSIS_DEBUG_Appendf                 StringBuilder.Appendf
	#define UE_TRACE_ANALYSIS_DEBUG_AppendChar              StringBuilder.AppendChar
	#define UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder()      printf("%s\n", StringBuilder.ToString())
	#define UE_TRACE_ANALYSIS_DEBUG_ResetStringBuilder()    StringBuilder.Reset()

#elif UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL == 3 // GLog

	#include "Containers/StringConv.h"
	#include "CoreGlobals.h"
	#include "Misc/OutputDeviceRedirector.h"
	#include "Misc/StringBuilder.h"

	#define UE_TRACE_ANALYSIS_DEBUG_LOG(format, ...)        { TAnsiStringBuilder<1024> SB; SB.Appendf(format, ##__VA_ARGS__); GLog->Logf(TEXT("%hs"), SB.ToString()); }
	#define UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder()    TAnsiStringBuilder<1024> StringBuilder;
	#define UE_TRACE_ANALYSIS_DEBUG_Append                  StringBuilder.Append
	#define UE_TRACE_ANALYSIS_DEBUG_Appendf                 StringBuilder.Appendf
	#define UE_TRACE_ANALYSIS_DEBUG_AppendChar              StringBuilder.AppendChar
	#define UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder()      GLog->Logf(TEXT("%hs"), StringBuilder.ToString())
	#define UE_TRACE_ANALYSIS_DEBUG_ResetStringBuilder()    StringBuilder.Reset()

#elif UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL == 4 // FPlatformMisc::LowLevelOutputDebugStringf

	#include "Misc/StringBuilder.h"

	#define UE_TRACE_ANALYSIS_DEBUG_LOG(format, ...)        { TAnsiStringBuilder<1024> SB; SB.Appendf(format, ##__VA_ARGS__); FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%hs\n"), SB.ToString()); }
	#define UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder()
	#define UE_TRACE_ANALYSIS_DEBUG_Append(s)               FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%hs"), s)
	#define UE_TRACE_ANALYSIS_DEBUG_Appendf(format, ...)    { TAnsiStringBuilder<1024> SB; SB.Appendf(format, ##__VA_ARGS__); FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%hs"), SB.ToString()); }
	#define UE_TRACE_ANALYSIS_DEBUG_AppendChar(c)           FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%hc"), c)
	#define UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder()      FPlatformMisc::LowLevelOutputDebugString(TEXT("\n"))
	#define UE_TRACE_ANALYSIS_DEBUG_ResetStringBuilder()

#endif // UE_TRACE_ANALYSIS_DEBUG_LOG_IMPL

#else // UE_TRACE_ANALYSIS_DEBUG

	#define UE_TRACE_ANALYSIS_DEBUG_LOG(format, ...)
	#define UE_TRACE_ANALYSIS_DEBUG_BeginStringBuilder()
	#define UE_TRACE_ANALYSIS_DEBUG_Append(s)
	#define UE_TRACE_ANALYSIS_DEBUG_Appendf(format, ...)
	#define UE_TRACE_ANALYSIS_DEBUG_AppendChar(c)
	#define UE_TRACE_ANALYSIS_DEBUG_EndStringBuilder()
	#define UE_TRACE_ANALYSIS_DEBUG_ResetStringBuilder()

#endif // UE_TRACE_ANALYSIS_DEBUG
#endif // !defined(UE_TRACE_ANALYSIS_DEBUG_LOG)

#if UE_TRACE_ANALYSIS_DEBUG

namespace UE::Trace::Analysis
{
	// Log level (1 = minimum, 2 = normal, 3 = verbose, 4 = verbose+).
	// It is used only when UE_TRACE_ANALYSIS_DEBUG is enabled.
	extern TRACEANALYSIS_API uint32 GDebugLevel;
}

#define UE_TRACE_ANALYSIS_DEBUG_LOG_EX(level, format, ...) \
	{ \
		if (UE::Trace::Analysis::GDebugLevel >= level) \
		{ \
			UE_TRACE_ANALYSIS_DEBUG_LOG(format, __VA_ARGS__); \
		} \
	}

#else // UE_TRACE_ANALYSIS_DEBUG

#define UE_TRACE_ANALYSIS_DEBUG_LOG_EX(level, format, ...)

#endif // UE_TRACE_ANALYSIS_DEBUG
