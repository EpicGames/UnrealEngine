// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceAnalysis : ModuleRules
{
	public TraceAnalysis(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Asio",
				"Cbor",
				"Core",
				"OpenSSL",
				"Sockets",
				"TraceLog",
			}
		);

        if (Target.bCompileAsioSSLSupport)
        {
	        PublicDefinitions.Add("WITH_SHARED_ASIO_SSL_SUPPORT=1");
        }
        else
        {
	        PublicDefinitions.Add("WITH_SHARED_ASIO_SSL_SUPPORT=0");
        }
	}
}
